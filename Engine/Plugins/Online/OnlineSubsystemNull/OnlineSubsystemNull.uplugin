{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem NULL",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to NULL platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,

    "Modules": [
        {
            "Name": "OnlineSubsystemNull",
            "Type": "Runtime",
            "LoadingPhase" : "Default"
        }
    ],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		}
	]

}