// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemNull : ModuleRules
{
	public OnlineSubsystemNull(ReadOnlyTargetRules Target) : base(Target)
    {
		PrivateDefinitions.Add("ONLINESUBSYSTEMNULL_PACKAGE=1");
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;


        if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Engine"
                }
            );

            PublicDependencyModuleNames.AddRange(
			   new string[] {
					"OnlineSubsystemUtils"
			   }
		   );
		}

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Sockets",
				"CoreUObject",
				"OnlineBase",
				"OnlineSubsystem", 
				"Json"
			}
		);
	}
}
