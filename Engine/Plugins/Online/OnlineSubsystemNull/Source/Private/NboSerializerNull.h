// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemNullTypes.h"
#include "NboSerializerOSS.h"

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBufferNull : public FNboSerializeToBufferOSS
{
public:
	/** Default constructor zeros num bytes*/
	FNboSerializeToBufferNull() :
		FNboSerializeToBufferOSS(512)
	{
	}

	/** Constructor specifying the size to use */
	FNboSerializeToBufferNull(uint32 Size) :
		FNboSerializeToBufferOSS(Size)
	{
	}

	/**
	 * Adds Null session info to the buffer
	 */
 	friend inline FNboSerializeToBufferNull& operator<<(FNboSerializeToBufferNull& Ar, const FOnlineSessionInfoNull& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());
		// Skip SessionType (assigned at creation)
		Ar << *SessionInfo.SessionId;
		((FNboSerializeToBuffer&)Ar) << *SessionInfo.HostAddr;
		return Ar;
 	}

	/**
	 * Adds Null Unique Id to the buffer
	 */
	friend inline FNboSerializeToBufferNull& operator<<(FNboSerializeToBufferNull& Ar, const FUniqueNetIdNull& UniqueId)
	{
		((FNboSerializeToBuffer&)Ar) << UniqueId.UniqueNetIdStr;
		return Ar;
	}
};

/**
 * Class used to write data into packets for sending via system link
 */
class FNboSerializeFromBufferNull : public FNboSerializeFromBufferOSS
{
public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 */
	FNboSerializeFromBufferNull(uint8* Packet,int32 Length) :
		FNboSerializeFromBufferOSS(Packet,Length)
	{
	}

	/**
	 * Reads Null session info from the buffer
	 */
 	friend inline FNboSerializeFromBufferNull& operator>>(FNboSerializeFromBufferNull& Ar, FOnlineSessionInfoNull& SessionInfo)
 	{
		check(SessionInfo.HostAddr.IsValid());
		// Skip SessionType (assigned at creation)
		SessionInfo.SessionId = FUniqueNetIdNull::Create();
		Ar >> const_cast<FUniqueNetIdNull&>(*SessionInfo.SessionId);
		Ar >> *SessionInfo.HostAddr;
		return Ar;
 	}

	/**
	 * Reads Null Unique Id from the buffer
	 */
	friend inline FNboSerializeFromBufferNull& operator>>(FNboSerializeFromBufferNull& Ar, FUniqueNetIdNull& UniqueId)
	{
		Ar >> UniqueId.UniqueNetIdStr;
		return Ar;
	}
};
