// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAchievementsInterfaceNull.h"
#include "OnlineSubsystem.h" // IWYU pragma: keep
#include "OnlineSubsystemNullTypes.h" // IWYU pragma: keep

FOnlineAchievementsNull::FOnlineAchievementsNull(class FOnlineSubsystemNull* InSubsystem)
	:	NullSubsystem(InSubsystem)
{
	check(NullSubsystem);
}

bool FOnlineAchievementsNull::ReadAchievementsFromConfig()
{
	if (Achievements.Num() > 0)
	{
		return true;
	}

	NullAchievementsConfig Config;
	return Config.ReadAchievements(Achievements);
}

void FOnlineAchievementsNull::WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	// treat each achievement as unlocked
	const int32 AchNum = PlayerAch->Num();
	for (FStatPropertyArray::TConstIterator It(WriteObject->Properties); It; ++It)
	{
		const FString AchievementId = It.Key();
		for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
		{
			if ((*PlayerAch)[ AchIdx ].Id == AchievementId)
			{
				TriggerOnAchievementUnlockedDelegates(PlayerId, AchievementId);
				break;
			}
		}
	}

	WriteObject->WriteState = EOnlineAsyncTaskState::Done;
	Delegate.ExecuteIfBound(PlayerId, true);
};

void FOnlineAchievementsNull::QueryAchievements( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate )
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	if (!PlayerAchievements.Find(PlayerId.AsShared()))
	{
		// copy for a new player
		TArray<FOnlineAchievement> AchievementsForPlayer;
		const int32 AchNum = Achievements.Num();

		for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
		{
			AchievementsForPlayer.Add( Achievements[ AchIdx ] );
		}

		PlayerAchievements.Add(PlayerId.AsShared(), AchievementsForPlayer);
	}

	Delegate.ExecuteIfBound(PlayerId, true);
}

void FOnlineAchievementsNull::QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate )
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	if (AchievementDescriptions.Num() == 0)
	{
		const int32 AchNum = Achievements.Num();
		for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
		{
			AchievementDescriptions.Add(Achievements[AchIdx].Id, Achievements[AchIdx]);
		}

		check(AchievementDescriptions.Num() > 0);
	}

	Delegate.ExecuteIfBound(PlayerId, true);
}

EOnlineCachedResult::Type FOnlineAchievementsNull::GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		return EOnlineCachedResult::NotFound;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		return EOnlineCachedResult::NotFound;
	}

	const int32 AchNum = PlayerAch->Num();
	for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
	{
		if ((*PlayerAch)[ AchIdx ].Id == AchievementId)
		{
			OutAchievement = (*PlayerAch)[ AchIdx ];
			return EOnlineCachedResult::Success;
		}
	}

	// no such achievement
	return EOnlineCachedResult::NotFound;
};

EOnlineCachedResult::Type FOnlineAchievementsNull::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements)
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		return EOnlineCachedResult::NotFound;
	}

	const TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		return EOnlineCachedResult::NotFound;
	}

	OutAchievements = *PlayerAch;
	return EOnlineCachedResult::Success;
};

EOnlineCachedResult::Type FOnlineAchievementsNull::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		return EOnlineCachedResult::NotFound;
	}

	if (AchievementDescriptions.Num() == 0 )
	{
		// don't have descs
		return EOnlineCachedResult::NotFound;
	}

	FOnlineAchievementDesc * AchDesc = AchievementDescriptions.Find(AchievementId);
	if (NULL == AchDesc)
	{
		// no such achievement
		return EOnlineCachedResult::NotFound;
	}

	OutAchievementDesc = *AchDesc;
	return EOnlineCachedResult::Success;
};

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsNull::ResetAchievements(const FUniqueNetId& PlayerId)
{
	if (!ReadAchievementsFromConfig())
	{
		// we don't have achievements
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("No achievements have been configured"));
		return false;
	}

	TArray<FOnlineAchievement> * PlayerAch = PlayerAchievements.Find(PlayerId.AsShared());
	if (NULL == PlayerAch)
	{
		// achievements haven't been read for a player
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Could not find achievements for player %s"), *PlayerId.ToString());
		return false;
	}

	// treat each achievement as unlocked
	const int32 AchNum = PlayerAch->Num();
	for (int32 AchIdx = 0; AchIdx < AchNum; ++AchIdx)
	{
		(*PlayerAch)[ AchIdx ].Progress = 0.0;
	}

	return true;
};
#endif // !UE_BUILD_SHIPPING
