// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/OnlineAchievementsInterface.h"

/**
 *	IOnlineAchievements - Interface class for acheivements
 */
class FOnlineAchievementsNull : public IOnlineAchievements
{
private:

	/** Structure describing a Null achievement */
	struct FOnlineAchievementNull : public FOnlineAchievement, public FOnlineAchievementDesc
	{
	};

	/**
	 * A helper class for configuring achievements in ini
	 */
	struct NullAchievementsConfig
	{
		/** 
		 * Create a config using the default values:
		 * IniName - GEngineIni
		 */
		NullAchievementsConfig()
			:	IniName(GEngineIni)
			,	SectionName(TEXT("OnlineSubsystemNull"))
		{
		}

		/** Returns empty string if couldn't read */
		FString GetKey(const FString& KeyName)
		{
			FString Result;
			if (!GConfig->GetString(*SectionName, *KeyName, Result, IniName))
			{
				return TEXT("");	// could just return Result, but being explicit is better
			}
			return Result;
		}

		bool ReadAchievements(TArray<FOnlineAchievementNull> & OutArray)
		{
			OutArray.Empty();
			int NumAchievements = 0;

			for(;;++NumAchievements)
			{
				FString Id = GetKey(FString::Printf(TEXT("Achievement_%d_Id"), NumAchievements));
				if (Id.IsEmpty())
				{
					break;
				}

				FOnlineAchievementNull NewAch;
				NewAch.Id = Id;
				NewAch.Progress = 0.0;

				NewAch.bIsHidden = GetKey(FString::Printf(TEXT("Achievement_%d_bIsHidden"), NumAchievements)) == TEXT("true");
				// these strings could be passed to Localize() first... but is it worth it for Null?
				NewAch.Title = FText::FromString(GetKey(FString::Printf(TEXT("Achievement_%d_Title"), NumAchievements)));
				NewAch.LockedDesc = FText::FromString(GetKey(FString::Printf(TEXT("Achievement_%d_LockedDesc"), NumAchievements)));
				NewAch.UnlockedDesc = FText::FromString(GetKey(FString::Printf(TEXT("Achievement_%d_UnlockedDesc"), NumAchievements)));
				
				OutArray.Add(NewAch);
			}

			return NumAchievements > 0;
		}


		/** Ini file name to find the config values */
		FString IniName;
		/** Section name for Null */
		FString SectionName;
	};

	/** Reference to the owning subsystem */
	class FOnlineSubsystemNull* NullSubsystem;

	/** hide the default constructor, we need a reference to our OSS */
	FOnlineAchievementsNull() {};

	/** Mapping of players to their achievements */
	TUniqueNetIdMap<TArray<FOnlineAchievement>> PlayerAchievements;

	/** Cached achievement descriptions for an Id */
	TMap<FString, FOnlineAchievementDesc> AchievementDescriptions;

	/** Cached achievements (not player-specific) */
	TArray<FOnlineAchievementNull> Achievements;

	/** Initializes achievements from config. Returns true if there is at least one achievement */
	bool ReadAchievementsFromConfig();

public:

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineAchievementsNull(class FOnlineSubsystemNull* InSubsystem);

	/**
	 * Default destructor
	 */
	virtual ~FOnlineAchievementsNull()= default;

	//~ Begin IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override;
	virtual void QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate() ) override;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements) override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineAchievements Interface
};
