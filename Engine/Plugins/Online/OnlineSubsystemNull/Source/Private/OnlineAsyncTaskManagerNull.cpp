// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskManagerNull.h"

void FOnlineAsyncTaskManagerNull::OnlineTick()
{
	check(NullSubsystem);
	check(FPlatformTLS::GetCurrentThreadId() == OnlineThreadId || !FPlatformProcess::SupportsMultithreading());
}

