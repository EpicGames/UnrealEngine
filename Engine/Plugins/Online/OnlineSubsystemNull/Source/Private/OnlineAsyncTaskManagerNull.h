// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"

/**
 *	Null version of the async task manager to register the various Null callbacks with the engine
 */
class FOnlineAsyncTaskManagerNull : public FOnlineAsyncTaskManager
{
protected:

	/** Cached reference to the main online subsystem */
	class FOnlineSubsystemNull* NullSubsystem;

public:

	FOnlineAsyncTaskManagerNull(class FOnlineSubsystemNull* InOnlineSubsystem)
		: NullSubsystem(InOnlineSubsystem)
	{
	}

	~FOnlineAsyncTaskManagerNull() = default; 

	// FOnlineAsyncTaskManager
	virtual void OnlineTick() override;
};
