// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUINull.h"
#include "OnlineSubsystemNull.h"
#include "OnlineIdentityNull.h"

FOnlineExternalUINull::FOnlineExternalUINull(FOnlineSubsystemNull* InSubsystem)
	: NullSubsystem(InSubsystem)
{

}

FOnlineExternalUINull::~FOnlineExternalUINull()
{

}

bool FOnlineExternalUINull::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	NullSubsystem->ExecuteNextTick([this, ControllerIndex, Delegate]()
	{
		// Just call identity login, possibly with a different user than initially
		int32 NewUserIndex = ControllerIndex;
		if (FOnlineSubsystemNull::bForceShowLoginUIUserChange)
		{
			NewUserIndex++;
		}

		FOnlineIdentityNullPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityNull>(NullSubsystem->GetIdentityInterface());
		if (IdentityInt.IsValid())
		{
			FOnLoginCompleteDelegate CompletionDelegate;
			CompletionDelegate = FOnLoginCompleteDelegate::CreateRaw(this, &FOnlineExternalUINull::OnIdentityLoginComplete, Delegate);
			IdentityInt->LoginInternal(NewUserIndex, FOnlineAccountCredentials(TEXT("ShowLoginUI"), TEXT("DummyUser"), TEXT("DummyId")), CompletionDelegate);
		}
	});

	return true;
}

void FOnlineExternalUINull::OnIdentityLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate)
{
	FUniqueNetIdPtr StrongUserId = UserId.AsShared();
	NullSubsystem->ExecuteNextTick([StrongUserId, LocalUserNum, bWasSuccessful, Delegate]()
	{
		Delegate.ExecuteIfBound(StrongUserId, LocalUserNum, FOnlineError(bWasSuccessful));
	});
}

bool FOnlineExternalUINull::ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUINull::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUINull::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUINull::ShowAchievementsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUINull::ShowLeaderboardUI( const FString& LeaderboardName )
{
	return false;
}

bool FOnlineExternalUINull::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUINull::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUINull::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUINull::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUINull::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUINull::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	return false;
}
