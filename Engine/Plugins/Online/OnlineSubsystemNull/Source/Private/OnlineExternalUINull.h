// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "OnlineSubsystemTypes.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemNullTypes.h"
 
class FOnlineSubsystemNull;

/** 
 * Null implementation to allow testing
 */
class FOnlineExternalUINull : public IOnlineExternalUI
{
public:

	/**
	 * Constructor
	 * @param InSubsystem The owner of this external UI interface.
	 */
	FOnlineExternalUINull(FOnlineSubsystemNull* InSubsystem);

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUINull();

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) override;
	virtual bool ShowFriendsUI(int32 LocalUserNum) override;
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName = NAME_GameSession) override;
	virtual bool ShowAchievementsUI(int32 LocalUserNum) override;
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) override;
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) override;
	virtual bool CloseWebURL() override;
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) override;
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) override;
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) override;
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;

private:

	/** Reference to the owning subsystem */
	FOnlineSubsystemNull *NullSubsystem;

	void OnIdentityLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error, FOnLoginUIClosedDelegate Delegate);
};

typedef TSharedPtr<FOnlineExternalUINull, ESPMode::ThreadSafe> FOnlineExternalUINullPtr;

