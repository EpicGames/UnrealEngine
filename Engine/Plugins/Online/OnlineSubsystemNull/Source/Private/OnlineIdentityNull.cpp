// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityNull.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Misc/OutputDeviceRedirector.h"
#include "OnlineSubsystemNull.h"
#include "SocketSubsystem.h"
#include "OnlineError.h"

#define USER_ATTR_NULL_LOGINCOUNT TEXT("null:logincount")

bool FUserOnlineAccountNull::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* FoundAttr = AdditionalAuthData.Find(AttrName);
	if (FoundAttr != NULL)
	{
		OutAttrValue = *FoundAttr;
		return true;
	}
	return false;
}

bool FUserOnlineAccountNull::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* FoundAttr = UserAttributes.Find(AttrName);
	if (FoundAttr != NULL)
	{
		OutAttrValue = *FoundAttr;
		return true;
	}
	return false;
}

bool FUserOnlineAccountNull::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	const FString* FoundAttr = UserAttributes.Find(AttrName);
	if (FoundAttr == NULL || *FoundAttr != AttrValue)
	{
		UserAttributes.Add(AttrName, AttrValue);
		return true;
	}
	return false;
}

FString FOnlineIdentityNull::GenerateRandomUserId(int32 LocalUserNum)
{
	FString HostName;
	if (!ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetHostName(HostName))
	{
		// could not get hostname, use address
		bool bCanBindAll;
		TSharedPtr<class FInternetAddr> Addr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);
		HostName = Addr->ToString(false);
	}

	bool bUseStableNullId = FOnlineSubsystemNull::bForceStableNullId;
	FString UserSuffix;

	if (FOnlineSubsystemNull::bAddUserNumToNullId)
	{
		UserSuffix = FString::Printf(TEXT("-%d"), LocalUserNum);
	}

	if (FPlatformProcess::IsFirstInstance() && !GIsEditor)
	{
		// If we're outside the editor and know this is the first instance, use the system login id
		bUseStableNullId = true;
	}

	if (bUseStableNullId)
	{
		// Use a stable id possibly with a user num suffix
		return FString::Printf(TEXT( "%s-%s%s" ), *HostName, *FPlatformMisc::GetLoginId().ToUpper(), *UserSuffix);
	}

	// If we're not the first instance (or in the editor), return truly random id
	return FString::Printf(TEXT( "%s-%s%s" ), *HostName, *FGuid::NewGuid().ToString(), *UserSuffix);
}

bool FOnlineIdentityNull::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	return LoginInternal(LocalUserNum, AccountCredentials, FOnLoginCompleteDelegate::CreateLambda(
		[this](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorStr)
		{
			TriggerOnLoginCompleteDelegates(LocalUserNum, bWasSuccessful, UserId, ErrorStr);
		}));
}

bool FOnlineIdentityNull::LoginInternal(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials, const FOnLoginCompleteDelegate& InCompletionDelegate)
{
	FString ErrorStr;
	TSharedPtr<FUserOnlineAccountNull> UserAccountPtr;

	// valid local player index
	if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
	{
		ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
	}
	else if (FOnlineSubsystemNull::bRequireShowLoginUI && !AccountCredentials.Type.Contains(TEXT("LoginUI")))
	{
		ErrorStr = TEXT("Only supports login using ShowLoginUI");
	}
	else if (FOnlineSubsystemNull::bRequireLoginCredentials && AccountCredentials.Id.IsEmpty())
	{
		ErrorStr = TEXT("Invalid account id, string empty");
	}
	else
	{
		FUniqueNetIdPtr* UserId = UserIds.Find(LocalUserNum);
		if (UserId == nullptr)
		{
			FString RandomUserId = GenerateRandomUserId(LocalUserNum);

			const FUniqueNetIdRef NewUserId = FUniqueNetIdNull::Create(RandomUserId);
			UserAccountPtr = MakeShareable(new FUserOnlineAccountNull(RandomUserId));
			UserAccountPtr->UserAttributes.Add(USER_ATTR_ID, RandomUserId);

			// update/add cached entry for user
			UserAccounts.Add(NewUserId, UserAccountPtr.ToSharedRef());

			// keep track of user ids for local users
			UserIds.Add(LocalUserNum, UserAccountPtr->GetUserId());
		}
		else
		{
			UserAccountPtr = UserAccounts.FindChecked(UserId->ToSharedRef());
		}

		// Increment the login count
		FString LoginCountString = TEXT("0");
		UserAccountPtr->GetUserAttribute(USER_ATTR_NULL_LOGINCOUNT, LoginCountString);
		int32 LoginCount = FCString::Atoi(*LoginCountString) + 1;
		LoginCountString = FString::Printf(TEXT("%d"), LoginCount);
		UserAccountPtr->SetUserAttribute(USER_ATTR_NULL_LOGINCOUNT, LoginCountString);
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Login request failed. %s"), *ErrorStr);
		InCompletionDelegate.Execute(LocalUserNum, false, *FUniqueNetIdNull::EmptyId(), ErrorStr);
		return false;
	}

	InCompletionDelegate.Execute(LocalUserNum, true, *UserAccountPtr->GetUserId(), ErrorStr);
	return true;
}

bool FOnlineIdentityNull::Logout(int32 LocalUserNum)
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		// remove cached user account
		UserAccounts.Remove(UserId.ToSharedRef());
		// remove cached user id
		UserIds.Remove(LocalUserNum);
		// not async but should call completion delegate anyway
		TriggerOnLogoutCompleteDelegates(LocalUserNum, true);

		return true;
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."),
			LocalUserNum);
		TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
	}
	return false;
}

bool FOnlineIdentityNull::AutoLogin(int32 LocalUserNum)
{
	FString LoginStr;
	FString PasswordStr;
	FString TypeStr;

	FParse::Value(FCommandLine::Get(), TEXT("AUTH_LOGIN="), LoginStr);
	FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), PasswordStr);
	FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), TypeStr);

	bool bEnableWarning = LoginStr.Len() > 0 || PasswordStr.Len() > 0 || TypeStr.Len() > 0;
	
	if (!LoginStr.IsEmpty())
	{
		if (!PasswordStr.IsEmpty())
		{
			if (!TypeStr.IsEmpty())
			{
				return Login(LocalUserNum, FOnlineAccountCredentials(TypeStr, LoginStr, PasswordStr));
			}
			else if (bEnableWarning)
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("AutoLogin missing AUTH_TYPE=<type>."));
			}
		}
		else if (bEnableWarning)
		{
			UE_LOG_ONLINE_IDENTITY(Warning, TEXT("AutoLogin missing AUTH_PASSWORD=<password>."));
		}
	}
	else if (!FOnlineSubsystemNull::bRequireLoginCredentials)
	{
		// Act like a console and login with empty auth
		return Login(LocalUserNum, FOnlineAccountCredentials());
	}
	else if (bEnableWarning)
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("AutoLogin missing AUTH_LOGIN=<login id>."));
	}

	return false;
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityNull::GetUserAccount(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccount> Result;

	if (const TSharedRef<FUserOnlineAccountNull>* FoundUserAccount = UserAccounts.Find(UserId.AsShared()))
	{
		Result = *FoundUserAccount;
	}

	return Result;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityNull::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	
	for (TUniqueNetIdMap<TSharedRef<FUserOnlineAccountNull>>::TConstIterator It(UserAccounts); It; ++It)
	{
		Result.Add(It.Value());
	}

	return Result;
}

FUniqueNetIdPtr FOnlineIdentityNull::GetUniquePlayerId(int32 LocalUserNum) const
{
	const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);
	if (FoundId != NULL)
	{
		return *FoundId;
	}
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentityNull::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes != NULL && Size > 0)
	{
		FString StrId = FString::ConstructFromPtrSize((TCHAR*)Bytes, Size);
		return FUniqueNetIdNull::Create(StrId);
	}
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentityNull::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdNull::Create(Str);
}

ELoginStatus::Type FOnlineIdentityNull::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityNull::GetLoginStatus(const FUniqueNetId& UserId) const 
{
	TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(UserId);
	if (UserAccount.IsValid() &&
		UserAccount->GetUserId()->IsValid())
	{
		if (FOnlineSubsystemNull::bForceOfflineMode)
		{
			return ELoginStatus::UsingLocalProfile;
		}
		else if (FOnlineSubsystemNull::bOnlineRequiresSecondLogin)
		{
			FString LoginCountString = TEXT("0");
			UserAccount->GetUserAttribute(USER_ATTR_NULL_LOGINCOUNT, LoginCountString);
			int32 LoginCount = FCString::Atoi(*LoginCountString);
			if (LoginCount < 2)
			{
				return ELoginStatus::UsingLocalProfile;
			}
		}
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FString FOnlineIdentityNull::GetPlayerNickname(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UniqueId = GetUniquePlayerId(LocalUserNum);
	if (UniqueId.IsValid())
	{
		return UniqueId->ToString();
	}

	return TEXT("NullUser");
}

FString FOnlineIdentityNull::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	return UserId.ToString();
}

FString FOnlineIdentityNull::GetAuthToken(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccount> UserAccount = GetUserAccount(*UserId);
		if (UserAccount.IsValid())
		{
			return UserAccount->GetAccessToken();
		}
	}
	return FString();
}

void FOnlineIdentityNull::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityNull::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	NullSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

FOnlineIdentityNull::FOnlineIdentityNull(FOnlineSubsystemNull* InSubsystem)
	: NullSubsystem(InSubsystem)
{	
	if (FOnlineSubsystemNull::bAutoLoginAtStartup)
	{
		// autologin the 0-th player
		Login(0, FOnlineAccountCredentials(TEXT("DummyType"), TEXT("DummyUser"), TEXT("DummyId")) );
	}
}

FOnlineIdentityNull::~FOnlineIdentityNull()
{
}

void FOnlineIdentityNull::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	if (FOnlineSubsystemNull::bForceOfflineMode && Privilege == EUserPrivileges::CanPlayOnline)
	{
		Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NetworkConnectionUnavailable);
	}
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}

FPlatformUserId FOnlineIdentityNull::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityNull::GetAuthType() const
{
	return TEXT("");
}
