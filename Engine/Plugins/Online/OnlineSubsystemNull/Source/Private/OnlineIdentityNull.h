// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemNullTypes.h"

class FOnlineSubsystemNull;

/**
 * Info associated with an user account generated by this online service
 */
class FUserOnlineAccountNull : 
	public FUserOnlineAccount
{

public:

	// FOnlineUser
	
	virtual FUniqueNetIdRef GetUserId() const override { return UserIdPtr; }
	virtual FString GetRealName() const override { return TEXT("DummyRealName"); }
	virtual FString GetDisplayName(const FString& Platform = FString()) const override  { return TEXT("DummyDisplayName"); }
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override { return TEXT("DummyAuthTicket"); }
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FUserOnlineAccountNull

	FUserOnlineAccountNull(const FString& InUserId=TEXT("")) 
		: UserIdPtr(FUniqueNetIdNull::Create(InUserId))
	{ }

	virtual ~FUserOnlineAccountNull() = default;

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserIdPtr;

        /** Additional key/value pair data related to auth */
	TMap<FString, FString> AdditionalAuthData;
        /** Additional key/value pair data related to user attribution */
	TMap<FString, FString> UserAttributes;
};

/**
 * Null service implementation of the online identity interface
 */
class FOnlineIdentityNull : public IOnlineIdentity
{
public:

	// IOnlineIdentity

	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;

	// FOnlineIdentityNull

	/**
	 * Constructor
	 *
	 * @param InSubsystem online subsystem being used
	 */
	FOnlineIdentityNull(FOnlineSubsystemNull* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityNull();

	/** Login and call arbitrary callback instead of registered one */
	bool LoginInternal(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials, const FOnLoginCompleteDelegate& InCompletionDelegate);

	/** Creates a unique id for a user, may be stable based on config/command line */
	FString GenerateRandomUserId(int32 LocalUserNum);

private:

	/**
	 * Should use the initialization constructor instead
	 */
	FOnlineIdentityNull() = delete;

	/** Cached pointer to owning subsystem */
	FOnlineSubsystemNull* NullSubsystem;

	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr> UserIds;

	/** Ids mapped to locally registered users */
	TUniqueNetIdMap<TSharedRef<FUserOnlineAccountNull>> UserAccounts;
};

typedef TSharedPtr<FOnlineIdentityNull, ESPMode::ThreadSafe> FOnlineIdentityNullPtr;
