// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineLeaderboardInterfaceNull.h"
#include "OnlineSubsystemNull.h"
#include "Interfaces/OnlineIdentityInterface.h"

bool FOnlineLeaderboardsNull::ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject)
{
	// Clear out any existing data
	ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
	ReadObject->Rows.Empty();

	const int32 NumPlayerIds = Players.Num();
	if (NumPlayerIds)
	{
		ReadObject->ReadState = EOnlineAsyncTaskState::Done;

		FLeaderboardNull* Leaderboard = Leaderboards.Find(ReadObject->LeaderboardName);
		if (Leaderboard)
		{
			for (int32 RowIdx=0; RowIdx<Leaderboard->Rows.Num(); ++RowIdx)
			{
				FUniqueNetIdPtr RowPlayerID = Leaderboard->Rows[RowIdx].PlayerId;
				for (int32 PlayerIdIdx = 0; PlayerIdIdx < NumPlayerIds; ++PlayerIdIdx)
				{
					FUniqueNetIdPtr PlayerID = Players[PlayerIdIdx];
					if (PlayerID.IsValid() && RowPlayerID.IsValid() && (*PlayerID.Get() == *RowPlayerID.Get()))
					{
						ReadObject->Rows.Add(Leaderboard->Rows[RowIdx]);
					}
				}
			}
		}

		// if there are no stats for specified PlayerIds, add empty rows
		for (int32 PlayerIdIdx=0; PlayerIdIdx < NumPlayerIds; ++PlayerIdIdx)
		{
			if (ReadObject->FindPlayerRecord(*Players[PlayerIdIdx]) == NULL)
			{
				// cannot have a better nickname here
				FOnlineStatsRow NewRow(Players[PlayerIdIdx]->ToString(), Players[PlayerIdIdx]);
				NewRow.Rank = -1;
				ReadObject->Rows.Add(NewRow);
			}
		}
	}

	TriggerOnLeaderboardReadCompleteDelegates((ReadObject->ReadState == EOnlineAsyncTaskState::Done) ? true : false);
	return true;
}

bool FOnlineLeaderboardsNull::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	TArray< FUniqueNetIdRef > FriendsList;

	// always add a UniqueNetId for local user
	check(NullSubsystem);
	IOnlineIdentityPtr Identity = NullSubsystem->GetIdentityInterface();
	if (Identity.IsValid())
	{
		if (Identity->GetUniquePlayerId(LocalUserNum).IsValid())
		{
			FriendsList.Add(Identity->GetUniquePlayerId(LocalUserNum).ToSharedRef());
		}
	}

	// add all known players
	FLeaderboardNull* Leaderboard = Leaderboards.Find(ReadObject->LeaderboardName);
	if (Leaderboard)
	{
		for (int32 UserIdx = 0; UserIdx < Leaderboard->Rows.Num(); ++UserIdx)
		{
			if (Leaderboard->Rows[UserIdx].PlayerId.IsValid())
			{
				FriendsList.AddUnique(Leaderboard->Rows[UserIdx].PlayerId.ToSharedRef());
			}
		}
	}

	return ReadLeaderboards(FriendsList, ReadObject);
}

bool FOnlineLeaderboardsNull::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsNull::ReadLeaderboardsAroundRank is currently not supported."));
	return false;
}
bool FOnlineLeaderboardsNull::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsNull::ReadLeaderboardsAroundUser is currently not supported."));
	return false;
}

void FOnlineLeaderboardsNull::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	// NOOP
}

bool FOnlineLeaderboardsNull::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	bool bWasSuccessful = true;

	int32 NumLeaderboards = WriteObject.LeaderboardNames.Num();
	for (int32 LeaderboardIdx = 0; LeaderboardIdx < NumLeaderboards; ++LeaderboardIdx)
	{
		// Will create or retrieve the leaderboards, triggering async calls as appropriate
		FLeaderboardNull* Leaderboard = FindOrCreateLeaderboard(WriteObject.LeaderboardNames[LeaderboardIdx], WriteObject.SortMethod, WriteObject.DisplayFormat);
		check(Leaderboard);

		FOnlineStatsRow* PlayerRow = Leaderboard->FindOrCreatePlayerRecord(Player);
		check(PlayerRow);

		for (FStatPropertyArray::TConstIterator It(WriteObject.Properties); It; ++It)
		{
			const FString& StatName = It.Key();
			const FVariantData& Stat = It.Value();
			FVariantData* ExistingStat = PlayerRow->Columns.Find(StatName);
			if (ExistingStat)
			{
				//@TODO: Add support for other types (variant doesn't define an ordering operator)
				bool bJustAssign = true;

				if ((ExistingStat->GetType() == Stat.GetType()) && (Stat.GetType() == EOnlineKeyValuePairDataType::Int32))
				{
					int32 NewValue;
					int32 OldValue;
					Stat.GetValue(NewValue);
					ExistingStat->GetValue(OldValue);

					switch (WriteObject.SortMethod)
					{
					case ELeaderboardSort::Ascending:
						bJustAssign = NewValue < OldValue;
						break;
					case ELeaderboardSort::Descending:
						bJustAssign = NewValue > OldValue;
						break;
					default:
						bJustAssign = true;
					}
				}

				if (bJustAssign)
				{
					*ExistingStat = Stat;
				}
			}
			else
			{
				PlayerRow->Columns.Add(StatName, Stat);
			}
		}
	}

	// Write has no delegates as of now
	return bWasSuccessful;
}

FOnlineLeaderboardsNull::FLeaderboardNull* FOnlineLeaderboardsNull::FindOrCreateLeaderboard(const FString& LeaderboardName, ELeaderboardSort::Type SortMethod, ELeaderboardFormat::Type DisplayFormat)
{
	FLeaderboardNull* Existing = Leaderboards.Find(LeaderboardName);
	if (Existing == NULL)
	{
		FLeaderboardNull NewLeaderboard;
		NewLeaderboard.LeaderboardName = LeaderboardName;
		Leaderboards.Add(LeaderboardName, NewLeaderboard);
	}

	check(Leaderboards.Find(LeaderboardName));
	return Leaderboards.Find(LeaderboardName);
}

bool FOnlineLeaderboardsNull::FlushLeaderboards(const FName& SessionName)
{
	TriggerOnLeaderboardFlushCompleteDelegates(SessionName, true);
	return true;
}

bool FOnlineLeaderboardsNull::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	// NOOP
	return false;
}

