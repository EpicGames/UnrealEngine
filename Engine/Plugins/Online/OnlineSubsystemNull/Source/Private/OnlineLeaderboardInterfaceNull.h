// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystemNullTypes.h"

class FOnlineSubsystemNull;

/**
 * Interface definition for the online services leaderboard services 
 */
class FOnlineLeaderboardsNull : public IOnlineLeaderboards
{
private:
	
	/** Internal representation of a leadboard */
	struct FLeaderboardNull : public FOnlineLeaderboardRead
	{
		/**
		 *	Retrieve a single record from the leaderboard for a given user
		 *
		 * @param UserId user id to retrieve a record for
		 * @return the requested user row or NULL if not found
		 */
		FOnlineStatsRow* FindOrCreatePlayerRecord(const FUniqueNetId& UserId)
		{
			FOnlineStatsRow* Row = FindPlayerRecord(UserId);
			if (Row == NULL)
			{
				// cannot have a better nickname here
				FOnlineStatsRow NewRow(UserId.ToString(), FUniqueNetIdNull::Create(UserId));
				NewRow.Rank = -1;
				Rows.Add(NewRow);
			}

			check(FindPlayerRecord(UserId));
			return FindPlayerRecord(UserId);
		}
	};

	/** Reference to the main Null subsystem */
	class FOnlineSubsystemNull* NullSubsystem;

	/** Leaderboards maintained by the subsystem */
	TMap<FString, FLeaderboardNull> Leaderboards;

	FOnlineLeaderboardsNull() : 
		NullSubsystem(NULL)
	{
	}

	/**
	 * Creates a Null leaderboard
	 *
	 * If the leaderboard already exists, the leaderboard data will still be retrieved
	 * @param LeaderboardName name of leaderboard to create
	 * @param SortMethod method the leaderboard scores will be sorted, ignored if leaderboard exists
	 * @param DisplayFormat type of data the leaderboard represents, ignored if leaderboard exists
	 */
	FLeaderboardNull* FindOrCreateLeaderboard(const FString& LeaderboardName, ELeaderboardSort::Type SortMethod, ELeaderboardFormat::Type DisplayFormat);

PACKAGE_SCOPE:

	FOnlineLeaderboardsNull(FOnlineSubsystemNull* InNullSubsystem) :
		NullSubsystem(InNullSubsystem)
	{
	}

public:

	virtual ~FOnlineLeaderboardsNull() = default;

	// IOnlineLeaderboards
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;
};

typedef TSharedPtr<FOnlineLeaderboardsNull, ESPMode::ThreadSafe> FOnlineLeaderboardsNullPtr;

