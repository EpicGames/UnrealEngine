// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineMessageSanitizerNull.h"
#include "OnlineSubsystem.h"

FMessageSanitizerNull::FMessageSanitizerNull()
	: RequestId(0)
{
}

FMessageSanitizerNull::~FMessageSanitizerNull()
{
	for (TPair<int32, FTSTicker::FDelegateHandle>& Pair : RequestIdHandleMap)
	{
		FTSTicker::RemoveTicker(Pair.Value);
	}
}

void FMessageSanitizerNull::SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate)
{
	float Delay = 3.0f;
	int32 ThisRequest = RequestId++;

	FTSTicker::FDelegateHandle TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, DisplayName, CompletionDelegate, ThisRequest](float) -> bool
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerNull_SanitizeDisplayName);
		FString CleanString;
		PerformSanitize(DisplayName, CleanString);
		CompletionDelegate.ExecuteIfBound(true, CleanString);
		UE_LOG_ONLINE(Verbose, TEXT("Completed MCP Test sanitize of string %s (result: %s)"), *DisplayName, *CleanString);
		RequestIdHandleMap.Remove(ThisRequest);
		return false;
	}), Delay);

	RequestIdHandleMap.Add(ThisRequest, TickerHandle);
}

void FMessageSanitizerNull::SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate)
{
	float Delay = 3.0f;
	int32 ThisRequest = RequestId++;

	FTSTicker::FDelegateHandle TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, DisplayNames, CompletionDelegate, ThisRequest](float) -> bool
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerNull_SanitizeDisplayNames);
		TArray<FString> OutStrings;
		for (const FString& String : DisplayNames)
		{
			FString CleanString;
			PerformSanitize(String, CleanString);
			OutStrings.Add(MoveTemp(CleanString));
		}
		CompletionDelegate.ExecuteIfBound(true, OutStrings);
		UE_LOG_ONLINE(Verbose, TEXT("Completed MCP Test sanitize of %d strings."), DisplayNames.Num());
		RequestIdHandleMap.Remove(ThisRequest);
		return false;
	}), Delay);

	RequestIdHandleMap.Add(ThisRequest, TickerHandle);
}

void FMessageSanitizerNull::PerformSanitize(const FString& RawString, FString& OutCleanString) const
{
	OutCleanString = RawString.Replace(TEXT("dirty"), TEXT("d***y"));
}

void FMessageSanitizerNull::QueryBlockedUser(int32 LocalUserNum, const FString& FromUserIdStr, const FString& FromPlatform, const FOnQueryUserBlockedResponse& CompletionDelegate)
{
	FBlockedQueryResult Result;
	Result.UserId = FromUserIdStr;
	Result.bIsBlocked = false;
	CompletionDelegate.ExecuteIfBound(Result);
}

void FMessageSanitizerNull::ResetBlockedUserCache()
{
}
