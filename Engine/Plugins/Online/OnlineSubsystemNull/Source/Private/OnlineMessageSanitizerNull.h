// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IMessageSanitizerInterface.h"
#include "Containers/Ticker.h"

/**
 * Implements a degenerate/dummy interface chat message sanitization system for testing
 */
class FMessageSanitizerNull
	: public IMessageSanitizer
{
public:

	// IMessageSanitizer
	virtual void SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate) override;
	virtual void SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate) override;
	virtual void QueryBlockedUser(int32 LocalUserNum, const FString& FromUserIdStr, const FString& FromPlatform, const FOnQueryUserBlockedResponse& InCompletionDelegate) override;
	virtual void ResetBlockedUserCache() override;
	// FMessageSanitizerNull

	FMessageSanitizerNull();
	virtual ~FMessageSanitizerNull();

private:
	int32 RequestId;

	void PerformSanitize(const FString& RawString, FString& OutCleanString) const;
	TMap<int32, FTSTicker::FDelegateHandle> RequestIdHandleMap;
};
