// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePurchaseInterfaceNull.h"
#include "OnlineSubsystemNull.h"
#include "OnlineError.h"

namespace
{
	static FPurchaseReceipt::FReceiptOfferEntry MakeReceiptOfferEntry(const FUniqueNetIdNull& NullUserId, const FString& Id, const FString& Name)
	{
		FPurchaseReceipt::FReceiptOfferEntry OfferEntry(FString(), Id, 1);
		{
			FPurchaseReceipt::FLineItemInfo LineItem;
			LineItem.ItemName = Name;
			LineItem.UniqueId = Id;
			OfferEntry.LineItems.Emplace(MoveTemp(LineItem));
		}

		return OfferEntry;
	}
}

#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.purchase"

FOnlinePurchaseNull::FOnlinePurchaseNull(FOnlineSubsystemNull& InNullSubsystem)
	: NullSubsystem(InNullSubsystem)
{
}

FOnlinePurchaseNull::~FOnlinePurchaseNull()
{
}

void FOnlinePurchaseNull::Tick()
{
	if (PendingPurchaseFailTime.IsSet() && PendingPurchaseDelegate.IsSet())
	{
		if (FPlatformTime::Seconds() > PendingPurchaseFailTime.GetValue())
		{
			FOnPurchaseCheckoutComplete Delegate = MoveTemp(PendingPurchaseDelegate.GetValue());
			PendingPurchaseDelegate.Reset();
			PendingPurchaseFailTime.Reset();

			Delegate.ExecuteIfBound(FOnlineError(TEXT("Checkout was cancelled or timed out")), MakeShared<FPurchaseReceipt>());
		}
	}
}

bool FOnlinePurchaseNull::IsAllowedToPurchase(const FUniqueNetId& UserId)
{
	return true;
}

bool FOnlinePurchaseNull::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{
	Error = FOnlineError(EOnlineErrorResult::Success);
	return true;
}

void FOnlinePurchaseNull::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	// Lambda to wrap calling our delegate with an error and logging the message
	auto CallDelegateError = [this, &Delegate](FString&& ErrorMessage)
	{
		NullSubsystem.ExecuteNextTick([Delegate, MovedErrorMessage = MoveTemp(ErrorMessage)]() mutable
		{
			UE_LOG_ONLINE(Error, TEXT("%s"), *MovedErrorMessage);

			const TSharedRef<FPurchaseReceipt> PurchaseReceipt = MakeShared<FPurchaseReceipt>();
			PurchaseReceipt->TransactionState = EPurchaseTransactionState::Failed;

			Delegate.ExecuteIfBound(FOnlineError(MoveTemp(MovedErrorMessage)), PurchaseReceipt);
		});
	};

	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there were no entries passed to purchase"));
		return;
	}
	else if (CheckoutRequest.PurchaseOffers.Num() != 1)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there were more than one entry passed to purchase. We currently only support one."));
		return;
	}

	check(CheckoutRequest.PurchaseOffers.IsValidIndex(0));
	const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Entry = CheckoutRequest.PurchaseOffers[0];

	if (Entry.Quantity != 1)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, purchase quantity not set to one. We currently only support one."));
		return;
	}

	if (Entry.OfferId.IsEmpty())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, OfferId is blank."));
		return;
	}

	const IOnlineStoreV2Ptr NullStoreInt = NullSubsystem.GetStoreV2Interface();

	TSharedPtr<FOnlineStoreOffer> NullOffer = NullStoreInt->GetOffer(Entry.OfferId);
	if (!NullOffer.IsValid())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, Could not find corresponding offer."));
		return;
	}

	if (PendingPurchaseDelegate.IsSet())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there was another purchase in progress."));
		return;
	}

	PendingPurchaseDelegate = Delegate;

	TWeakPtr<FOnlinePurchaseNull, ESPMode::ThreadSafe> WeakMe = AsShared();
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);

	NullSubsystem.ExecuteNextTick([NullUserId, NullOffer, WeakMe]
	{
		FOnlinePurchaseNullPtr StrongThis = WeakMe.Pin();
		if (StrongThis.IsValid())
		{
			StrongThis->CheckoutSuccessfully(NullUserId, NullOffer);
		}
	});
}

void FOnlinePurchaseNull::CheckoutSuccessfully(const FUniqueNetIdNull& UserId, TSharedPtr<FOnlineStoreOffer> Offer)
{
	// Cache this receipt
	TArray<FPurchaseReceipt>& UserReceipts = UserFakeReceipts.FindOrAdd(UserId);
	FPurchaseReceipt& PurchaseReceipt = UserReceipts.Emplace_GetRef();
	PurchaseReceipt.AddReceiptOffer(MakeReceiptOfferEntry(UserId, Offer->OfferId, Offer->Title.ToString()));

	check(PendingPurchaseDelegate.IsSet());

	// Have a pending purchase
	FOnPurchaseCheckoutComplete Delegate = MoveTemp(PendingPurchaseDelegate.GetValue());
	PendingPurchaseDelegate.Reset();
	PendingPurchaseFailTime.Reset();

	// Finish pending purchase
	Delegate.ExecuteIfBound(FOnlineError(true), MakeShared<FPurchaseReceipt>(PurchaseReceipt));
}

void FOnlinePurchaseNull::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	// Lambda to wrap calling our delegate with an error and logging the message
	auto CallDelegateError = [this, &Delegate](FString&& ErrorMessage)
	{
		NullSubsystem.ExecuteNextTick([Delegate, MovedErrorMessage = MoveTemp(ErrorMessage)]() mutable
		{
			UE_LOG_ONLINE(Error, TEXT("%s"), *MovedErrorMessage);

			Delegate.ExecuteIfBound(FOnlineError(MoveTemp(MovedErrorMessage)));
		});
	};

	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there were no entries passed to purchase"));
		return;
	}
	else if (CheckoutRequest.PurchaseOffers.Num() != 1)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there were more than one entry passed to purchase. We currently only support one."));
		return;
	}

	check(CheckoutRequest.PurchaseOffers.IsValidIndex(0));
	const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Entry = CheckoutRequest.PurchaseOffers[0];

	if (Entry.Quantity != 1)
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, purchase quantity not set to one. We currently only support one."));
		return;
	}

	if (Entry.OfferId.IsEmpty())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, OfferId is blank."));
		return;
	}

	const IOnlineStoreV2Ptr NullStoreInt = NullSubsystem.GetStoreV2Interface();

	TSharedPtr<FOnlineStoreOffer> NullOffer = NullStoreInt->GetOffer(Entry.OfferId);
	if (!NullOffer.IsValid())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, Could not find corresponding offer."));
		return;
	}

	if (PendingPurchaseHandle.IsValid())
	{
		CallDelegateError(TEXT("FOnlinePurchaseNull::Checkout failed, there was another purchase in progress."));
		return;
	}

	PendingPurchaseHandle = Delegate.GetHandle();

	TWeakPtr<FOnlinePurchaseNull, ESPMode::ThreadSafe> WeakMe = AsShared();
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);

	NullSubsystem.ExecuteNextTick([NullUserId, WeakMe, Delegate]
		{
			FOnlinePurchaseNullPtr StrongThis = WeakMe.Pin();
			if (StrongThis.IsValid())
			{
				StrongThis->CheckoutSuccessfully(NullUserId, Delegate);
			}
		});
}

void FOnlinePurchaseNull::CheckoutSuccessfully(const FUniqueNetIdNull& UserId, FOnPurchaseReceiptlessCheckoutComplete Delegate)
{
	// Reset pending purchase handle
	check(PendingPurchaseHandle.IsValid());
	PendingPurchaseHandle.Reset();
	
	// Finish pending purchase
	Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::Success));
}

void FOnlinePurchaseNull::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);
	TArray<FPurchaseReceipt>* UserReceipts = UserFakeReceipts.Find(NullUserId);
	if (UserReceipts)
	{
		for (const FPurchaseReceipt& UserReceipt : *UserReceipts)
		{
			for (const FPurchaseReceipt::FReceiptOfferEntry& ReceiptOffer : UserReceipt.ReceiptOffers)
			{
				if (ReceiptOffer.OfferId == ReceiptId)
				{
					UE_LOG_ONLINE(Log, TEXT("Consumption of Entitlement %s completed was successful"), *ReceiptId);
					return;
				}
			}
		}
	}

	UE_LOG_ONLINE(Verbose, TEXT("Didn't find receipt with id %s"), *ReceiptId);
}

void FOnlinePurchaseNull::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	TWeakPtr<FOnlinePurchaseNull, ESPMode::ThreadSafe> WeakMe = AsShared();
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);

	NullSubsystem.ExecuteNextTick([NullUserId, WeakMe, RedeemCodeRequest, Delegate]
	{
		FOnlinePurchaseNullPtr StrongThis = WeakMe.Pin();
		if (StrongThis.IsValid())
		{
			UE_LOG_ONLINE(Log, TEXT("FOnlinePurchaseNull::RedeemCode redeemed successfully"));

			// Cache this receipt
			TArray<FPurchaseReceipt>& UserReceipts = StrongThis->UserFakeReceipts.FindOrAdd(NullUserId);
			FPurchaseReceipt& PurchaseReceipt = UserReceipts.Emplace_GetRef();
			PurchaseReceipt.AddReceiptOffer(MakeReceiptOfferEntry(NullUserId, RedeemCodeRequest.Code, RedeemCodeRequest.Code));

			Delegate.ExecuteIfBound(FOnlineError(true), MakeShared<FPurchaseReceipt>(PurchaseReceipt));
		}
	});
}

void FOnlinePurchaseNull::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);
	if (!NullUserId.IsValid())
	{
		NullSubsystem.ExecuteNextTick([Delegate]
		{
			UE_LOG_ONLINE(Error, TEXT("FOnlinePurchaseNull::QueryReceipts user is invalid"));

			Delegate.ExecuteIfBound(FOnlineError(TEXT("User is invalid")));
		});
		return;
	}

	NullSubsystem.ExecuteNextTick([Delegate]
	{
		Delegate.ExecuteIfBound(FOnlineError(true));
	});
}

void FOnlinePurchaseNull::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	const FUniqueNetIdNull& NullUserId = static_cast<const FUniqueNetIdNull&>(UserId);

	const TArray<FPurchaseReceipt>* FoundReceipts = UserFakeReceipts.Find(NullUserId);
	if (FoundReceipts == nullptr)
	{
		OutReceipts.Empty();
	}
	else
	{
		OutReceipts = *FoundReceipts;
	}
}

void FOnlinePurchaseNull::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{

}
