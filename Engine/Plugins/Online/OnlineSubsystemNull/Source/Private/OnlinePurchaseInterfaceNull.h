// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemNullTypes.h"

class FOnlineStoreOffer;

class FOnlineSubsystemNull;

class FOnlinePurchaseNull
	: public IOnlinePurchase
	, public TSharedFromThis<FOnlinePurchaseNull, ESPMode::ThreadSafe>
{
public:
	FOnlinePurchaseNull(FOnlineSubsystemNull& InNullSubsystem);
	virtual ~FOnlinePurchaseNull();

	void Tick();

public:
	//~ Begin IOnlinePurchase Interface
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	//~ End IOnlinePurchase Interface

PACKAGE_SCOPE:
	void CheckoutSuccessfully(const FUniqueNetIdNull& UserId, TSharedPtr<FOnlineStoreOffer> Offer);
	void CheckoutSuccessfully(const FUniqueNetIdNull& UserId, FOnPurchaseReceiptlessCheckoutComplete Delegate);

PACKAGE_SCOPE:
	/** Pointer back to our parent subsystem */
	FOnlineSubsystemNull& NullSubsystem;

	/** Cached receipts information per user */
	TMap<FUniqueNetIdNull, TArray<FPurchaseReceipt> > UserFakeReceipts;

	/** Do we have a purchase currently in progress? */
	TOptional<FOnPurchaseCheckoutComplete> PendingPurchaseDelegate;

	TOptional<double> PendingPurchaseFailTime;

	FDelegateHandle PendingPurchaseHandle;
};

using FOnlinePurchaseNullPtr = TSharedPtr<FOnlinePurchaseNull, ESPMode::ThreadSafe>;
using FOnlinePurchaseNullRef = TSharedRef<FOnlinePurchaseNull, ESPMode::ThreadSafe>;
