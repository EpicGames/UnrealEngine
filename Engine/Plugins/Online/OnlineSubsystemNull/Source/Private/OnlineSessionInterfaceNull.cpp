// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionInterfaceNull.h"

#include "UObject/CoreNet.h"
#include "Online/OnlineBase.h"
#include "Interfaces/OnlineIdentityInterface.h"

#include "NboSerializerNull.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemNull.h"
#if WITH_ENGINE
#include "OnlineSubsystemUtils.h"
#endif //WITH_ENGINE
#include "SocketSubsystem.h"



FOnlineSessionInfoNull::FOnlineSessionInfoNull() :
	HostAddr(NULL),
	SessionId(FUniqueNetIdNull::EmptyId())
{
}

void FOnlineSessionInfoNull::Init(const FOnlineSubsystemNull& Subsystem)
{
	// Read the IP from the system
	bool bCanBindAll;
	HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);

	// The below is a workaround for systems that set hostname to a distinct address from 127.0.0.1 on a loopback interface.
	// See e.g. https://www.debian.org/doc/manuals/debian-reference/ch05.en.html#_the_hostname_resolution
	// and http://serverfault.com/questions/363095/why-does-my-hostname-appear-with-the-address-127-0-1-1-rather-than-127-0-0-1-in
	// Since we bind to 0.0.0.0, we won't answer on 127.0.1.1, so we need to advertise ourselves as 127.0.0.1 for any other loopback address we may have.
	uint32 HostIp = 0;
	HostAddr->GetIp(HostIp); // will return in host order
	// if this address is on loopback interface, advertise it as 127.0.0.1
	if ((HostIp & 0xff000000) == 0x7f000000)
	{
		HostAddr->SetIp(0x7f000001);	// 127.0.0.1
	}

	// Now set the port that was configured
#if WITH_ENGINE
	// jntodo: can we put anything here if we don't have engine code?
	HostAddr->SetPort(GetPortFromNetDriver(Subsystem.GetInstanceName()));
#endif //WITH_ENGINE

	FGuid OwnerGuid;
	FPlatformMisc::CreateGuid(OwnerGuid);
	SessionId = FUniqueNetIdNull::Create(OwnerGuid.ToString());
}

/**
 *	Async task for ending a Null online session
 */
class FOnlineAsyncTaskNullEndSession : public FOnlineAsyncTaskBasic<FOnlineSubsystemNull>
{
private:
	/** Name of session ending */
	FName SessionName;

public:
	FOnlineAsyncTaskNullEndSession(class FOnlineSubsystemNull* InSubsystem, FName InSessionName) :
		FOnlineAsyncTaskBasic(InSubsystem),
		SessionName(InSessionName)
	{
	}

	~FOnlineAsyncTaskNullEndSession()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskNullEndSession bWasSuccessful: %d SessionName: %s"), WasSuccessful(), *SessionName.ToString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
		if (Session)
		{
			Session->SessionState = EOnlineSessionState::Ended;
		}
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			SessionInt->TriggerOnEndSessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}
};

/**
 *	Async task for destroying a Null online session
 */
class FOnlineAsyncTaskNullDestroySession : public FOnlineAsyncTaskBasic<FOnlineSubsystemNull>
{
private:
	/** Name of session ending */
	FName SessionName;

public:
	FOnlineAsyncTaskNullDestroySession(class FOnlineSubsystemNull* InSubsystem, FName InSessionName) :
		FOnlineAsyncTaskBasic(InSubsystem),
		SessionName(InSessionName)
	{
	}

	~FOnlineAsyncTaskNullDestroySession()
	{
	}

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskNullDestroySession bWasSuccessful: %d SessionName: %s"), WasSuccessful(), *SessionName.ToString());
	}

	/**
	 * Give the async task time to do its work
	 * Can only be called on the async task manager thread
	 */
	virtual void Tick() override
	{
		bIsComplete = true;
		bWasSuccessful = true;
	}

	/**
	 * Give the async task a chance to marshal its data back to the game thread
	 * Can only be called on the game thread by the async task manager
	 */
	virtual void Finalize() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			FNamedOnlineSession* Session = SessionInt->GetNamedSession(SessionName);
			if (Session)
			{
				SessionInt->RemoveNamedSession(SessionName);
			}
		}
	}

	/**
	 *	Async task is given a chance to trigger it's delegates
	 */
	virtual void TriggerDelegates() override
	{
		IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
		if (SessionInt.IsValid())
		{
			SessionInt->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}
};

bool FOnlineSessionNull::CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	uint32 Result = ONLINE_FAIL;

	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == NULL)
	{
		// Create a new session and deep copy the game settings
		Session = AddNamedSession(SessionName, NewSessionSettings);
		check(Session);
		Session->bHosting = true;
		Session->SessionState = EOnlineSessionState::Creating;
		Session->NumOpenPrivateConnections = NewSessionSettings.NumPrivateConnections;
		Session->NumOpenPublicConnections = NewSessionSettings.NumPublicConnections;	// always start with full public connections, local player will register later

		Session->HostingPlayerNum = HostingPlayerNum;

		check(NullSubsystem);
		IOnlineIdentityPtr Identity = NullSubsystem->GetIdentityInterface();
		if (Identity.IsValid())
		{
			Session->OwningUserId = Identity->GetUniquePlayerId(HostingPlayerNum);
			Session->OwningUserName = Identity->GetPlayerNickname(HostingPlayerNum);
		}

		// if did not get a valid one, use just something
		if (!Session->OwningUserId.IsValid())
		{
			Session->OwningUserId = FUniqueNetIdNull::Create(FString::Printf(TEXT("%d"), HostingPlayerNum));
			Session->OwningUserName = FString(TEXT("NullUser"));
		}

		// Setup the host session info
		FOnlineSessionInfoNull* NewSessionInfo = new FOnlineSessionInfoNull();
		NewSessionInfo->Init(*NullSubsystem);
		Session->SessionInfo = MakeShareable(NewSessionInfo);

		Result = UpdateLANStatus();

		if (Result != ONLINE_IO_PENDING)
		{
			// Set the game state as pending (not started)
			Session->SessionState = EOnlineSessionState::Pending;

			if (Result != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		TriggerOnCreateSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS));
	}
	
	return Result == ONLINE_IO_PENDING || Result == ONLINE_SUCCESS;
}

bool FOnlineSessionNull::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// todo: use proper	HostingPlayerId
	return CreateSession(0, SessionName, NewSessionSettings);
}

bool FOnlineSessionNull::NeedsToAdvertise()
{
	UE::TScopeLock ScopeLock(SessionLock);

	bool bResult = false;
	for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		FNamedOnlineSession& Session = Sessions[SessionIdx];
		if (NeedsToAdvertise(Session))
		{
			bResult = true;
			break;
		}
	}

	return bResult;
}

bool FOnlineSessionNull::NeedsToAdvertise( FNamedOnlineSession& Session )
{
	// In Null, we have to imitate missing online service functionality, so we advertise:
	// a) LAN match with open public connections (same as usually)
	// b) Not started public LAN session (same as usually)
	// d) Joinable presence-enabled session that would be advertised with in an online service
	// (all of that only if we're server)
	return Session.SessionSettings.bShouldAdvertise && IsHost(Session) &&
		(
			(
			  Session.SessionSettings.bIsLANMatch && 			  
			  (Session.SessionState != EOnlineSessionState::InProgress || (Session.SessionSettings.bAllowJoinInProgress && Session.NumOpenPublicConnections > 0))
			) 
			||
			(
				Session.SessionSettings.bAllowJoinViaPresence || Session.SessionSettings.bAllowJoinViaPresenceFriendsOnly
			)
		);		
}

bool FOnlineSessionNull::IsSessionJoinable(const FNamedOnlineSession& Session) const
{
	const FOnlineSessionSettings& Settings = Session.SessionSettings;

	// LAN beacons are implicitly advertised.
	const bool bIsAdvertised = Settings.bShouldAdvertise || Settings.bIsLANMatch;
	const bool bIsMatchInProgress = Session.SessionState == EOnlineSessionState::InProgress;

	const bool bJoinableFromProgress = (!bIsMatchInProgress || Settings.bAllowJoinInProgress);

	const bool bAreSpacesAvailable = Session.NumOpenPublicConnections > 0;

	// LAN matches don't care about private connections / invites.
	// LAN matches don't care about presence information.
	return bIsAdvertised && bJoinableFromProgress && bAreSpacesAvailable;
}


uint32 FOnlineSessionNull::UpdateLANStatus()
{
	uint32 Result = ONLINE_SUCCESS;

	if ( NeedsToAdvertise() )
	{
		// set up LAN session
		if (LANSessionManager.GetBeaconState() == ELanBeaconState::NotUsingLanBeacon)
		{
			FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FOnlineSessionNull::OnValidQueryPacketReceived);
			if (!LANSessionManager.Host(QueryPacketDelegate))
			{
				Result = ONLINE_FAIL;

				LANSessionManager.StopLANSession();
			}
		}
	}
	else
	{
		if (LANSessionManager.GetBeaconState() != ELanBeaconState::Searching)
		{
			// Tear down the LAN beacon
			LANSessionManager.StopLANSession();
		}
	}

	return Result;
}

bool FOnlineSessionNull::StartSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;
	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't start a match multiple times
		if (Session->SessionState == EOnlineSessionState::Pending ||
			Session->SessionState == EOnlineSessionState::Ended)
		{
			// If this lan match has join in progress disabled, shut down the beacon
			Result = UpdateLANStatus();
			Session->SessionState = EOnlineSessionState::InProgress;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning,	TEXT("Can't start an online session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate
		TriggerOnStartSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	bool bWasSuccessful = true;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// @TODO ONLINE update LAN settings
		Session->SessionSettings = UpdatedSessionSettings;
		TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}

	return bWasSuccessful;
}

bool FOnlineSessionNull::EndSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't end a match that isn't in progress
		if (Session->SessionState == EOnlineSessionState::InProgress)
		{
			Session->SessionState = EOnlineSessionState::Ended;

			// If the session should be advertised and the lan beacon was destroyed, recreate
			Result = UpdateLANStatus();
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		if (Session)
		{
			Session->SessionState = EOnlineSessionState::Ended;
		}

		TriggerOnEndSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	uint32 Result = ONLINE_FAIL;
	// Find the session in question
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// The session info is no longer needed
		RemoveNamedSession(Session->SessionName);

		Result = UpdateLANStatus();
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
	}

#if WITH_ENGINE
	if (GetNumSessions() == 0)
	{
		IOnlineVoicePtr VoiceInt = NullSubsystem->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			if (!NullSubsystem->IsDedicated())
			{
				// Stop local talkers
				VoiceInt->UnregisterLocalTalkers();
			}

			// Stop remote voice 
			VoiceInt->RemoveAllRemoteTalkers();
		}
	}
#endif //WITH_ENGINE

	if (Result != ONLINE_IO_PENDING)
	{
		CompletionDelegate.ExecuteIfBound(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
		TriggerOnDestroySessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(this, SessionName, UniqueId);
}

bool FOnlineSessionNull::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("StartMatchmaking is not supported on this platform. Use FindSessions or FindSessionById."));
	TriggerOnMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionNull::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionNull::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CancelMatchmaking is not supported on this platform. Use CancelFindSessions."));
	TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
	return false;
}

bool FOnlineSessionNull::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	uint32 Return = ONLINE_FAIL;

	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid() && SearchSettings->SearchState != EOnlineAsyncTaskState::InProgress)
	{
		// Free up previous results
		SearchSettings->SearchResults.Empty();

		// Copy the search pointer so we can keep it around
		CurrentSessionSearch = SearchSettings;

		// remember the time at which we started search, as this will be used for a "good enough" ping estimation
		SessionSearchStartInSeconds = FPlatformTime::Seconds();

		// Check if its a LAN query
		Return = FindLANSession();

		if (Return == ONLINE_IO_PENDING)
		{
			SearchSettings->SearchState = EOnlineAsyncTaskState::InProgress;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring game search request while one is pending"));
		Return = ONLINE_IO_PENDING;
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// This function doesn't use the SearchingPlayerNum parameter, so passing in anything is fine.
	return FindSessions(0, SearchSettings);
}

bool FOnlineSessionNull::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegates)
{
	FOnlineSessionSearchResult EmptyResult;
	CompletionDelegates.ExecuteIfBound(0, false, EmptyResult);
	return true;
}

uint32 FOnlineSessionNull::FindLANSession()
{
	uint32 Return = ONLINE_IO_PENDING;

	// Recreate the unique identifier for this client
	GenerateNonce((uint8*)&LANSessionManager.LanNonce, 8);

	FOnValidResponsePacketDelegate ResponseDelegate = FOnValidResponsePacketDelegate::CreateRaw(this, &FOnlineSessionNull::OnValidResponsePacketReceived);
	FOnSearchingTimeoutDelegate TimeoutDelegate = FOnSearchingTimeoutDelegate::CreateRaw(this, &FOnlineSessionNull::OnLANSearchTimeout);

	FNboSerializeToBufferNull Packet(LAN_BEACON_MAX_PACKET_SIZE);
	LANSessionManager.CreateClientQueryPacket(Packet, LANSessionManager.LanNonce);
	if (LANSessionManager.Search(Packet, ResponseDelegate, TimeoutDelegate) == false)
	{
		Return = ONLINE_FAIL;

		FinalizeLANSearch();

		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
		
		// Just trigger the delegate as having failed
		TriggerOnFindSessionsCompleteDelegates(false);
	}
	return Return;
}

bool FOnlineSessionNull::CancelFindSessions()
{
	uint32 Return = ONLINE_FAIL;
	if (CurrentSessionSearch.IsValid() && CurrentSessionSearch->SearchState == EOnlineAsyncTaskState::InProgress)
	{
		// Make sure it's the right type
		Return = ONLINE_SUCCESS;

		FinalizeLANSearch();

		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Failed;
		CurrentSessionSearch = NULL;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't cancel a search that isn't in progress"));
	}

	if (Return != ONLINE_IO_PENDING)
	{
		TriggerOnCancelFindSessionsCompleteDelegates(true);
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	uint32 Return = ONLINE_FAIL;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	// Don't join a session if already in one or hosting one
	if (Session == NULL)
	{
		// Create a named session from the search result data
		Session = AddNamedSession(SessionName, DesiredSession.Session);
		Session->HostingPlayerNum = PlayerNum;

		// Create Internet or LAN match
		FOnlineSessionInfoNull* NewSessionInfo = new FOnlineSessionInfoNull();
		Session->SessionInfo = MakeShareable(NewSessionInfo);

		Return = JoinLANSession(PlayerNum, Session, &DesiredSession.Session);

		// turn off advertising on Join, to avoid clients advertising it over LAN
		Session->SessionSettings.bShouldAdvertise = false;

		if (Return != ONLINE_IO_PENDING)
		{
			if (Return != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) already exists, can't join twice"), *SessionName.ToString());
	}

	if (Return != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate as having failed
		TriggerOnJoinSessionCompleteDelegates(SessionName, Return == ONLINE_SUCCESS ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionNull::JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	// Assuming player 0 should be OK here
	return JoinSession(0, SessionName, DesiredSession);
}

bool FOnlineSessionNull::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	TArray<FOnlineSessionSearchResult> EmptySearchResult;
	TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, EmptySearchResult);
	return false;
};

bool FOnlineSessionNull::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	TArray<FOnlineSessionSearchResult> EmptySearchResult;
	TriggerOnFindFriendSessionCompleteDelegates(0, false, EmptySearchResult);
	return false;
}

bool FOnlineSessionNull::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	TArray<FOnlineSessionSearchResult> EmptySearchResult;
	TriggerOnFindFriendSessionCompleteDelegates(0, false, EmptySearchResult);
	return false;
}

bool FOnlineSessionNull::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	return false;
};

bool FOnlineSessionNull::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	return false;
}

bool FOnlineSessionNull::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	return false;
};

bool FOnlineSessionNull::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	// this function has to exist due to interface definition, but it does not have a meaningful implementation in Null subsystem
	return false;
}

uint32 FOnlineSessionNull::JoinLANSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	check(Session != nullptr);

	uint32 Result = ONLINE_FAIL;
	Session->SessionState = EOnlineSessionState::Pending;

	if (Session->SessionInfo.IsValid() && SearchSession != nullptr && SearchSession->SessionInfo.IsValid())
	{
		// Copy the session info over
		const FOnlineSessionInfoNull* SearchSessionInfo = (const FOnlineSessionInfoNull*)SearchSession->SessionInfo.Get();
		FOnlineSessionInfoNull* SessionInfo = (FOnlineSessionInfoNull*)Session->SessionInfo.Get();
		SessionInfo->SessionId = SearchSessionInfo->SessionId;

		SessionInfo->HostAddr = SearchSessionInfo->HostAddr->Clone();
		Result = ONLINE_SUCCESS;
	}

	return Result;
}

bool FOnlineSessionNull::PingSearchResults(const FOnlineSessionSearchResult& SearchResult)
{
	return false;
}

/** Get a resolved connection string from a session info */
static bool GetConnectStringFromSessionInfo(TSharedPtr<FOnlineSessionInfoNull>& SessionInfo, FString& ConnectInfo, int32 PortOverride=0)
{
	bool bSuccess = false;
	if (SessionInfo.IsValid())
	{
		if (SessionInfo->HostAddr.IsValid() && SessionInfo->HostAddr->IsValid())
		{
			if (PortOverride != 0)
			{
				ConnectInfo = FString::Printf(TEXT("%s:%d"), *SessionInfo->HostAddr->ToString(false), PortOverride);
			}
			else
			{
				ConnectInfo = FString::Printf(TEXT("%s"), *SessionInfo->HostAddr->ToString(true));
			}

			bSuccess = true;
		}
	}

	return bSuccess;
}

bool FOnlineSessionNull::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccess = false;
	// Find the session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session != NULL)
	{
		TSharedPtr<FOnlineSessionInfoNull> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoNull>(Session->SessionInfo);
		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(Session->SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}

		if (!bSuccess)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for session %s in GetResolvedConnectString()"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning,
			TEXT("Unknown session name (%s) specified to GetResolvedConnectString()"),
			*SessionName.ToString());
	}

	return bSuccess;
}

bool FOnlineSessionNull::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)
{
	bool bSuccess = false;
	if (SearchResult.Session.SessionInfo.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoNull> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoNull>(SearchResult.Session.SessionInfo);

		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(SearchResult.Session.SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);

		}
		else if (PortType == NAME_GamePort)
		{
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
		}
	}
	
	if (!bSuccess || ConnectInfo.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info in search result to GetResolvedConnectString()"));
	}

	return bSuccess;
}

FOnlineSessionSettings* FOnlineSessionNull::GetSessionSettings(FName SessionName) 
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		return &Session->SessionSettings;
	}
	return NULL;
}

void FOnlineSessionNull::RegisterLocalPlayers(FNamedOnlineSession* Session)
{
#if WITH_ENGINE
	if (!NullSubsystem->IsDedicated())
	{
		IOnlineVoicePtr VoiceInt = NullSubsystem->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			for (int32 Index = 0; Index < MAX_LOCAL_PLAYERS; Index++)
			{
				// Register the local player as a local talker
				VoiceInt->RegisterLocalTalker(Index);
			}
		}
	}
#endif //WITH_ENGINE
}

void FOnlineSessionNull::RegisterVoice(const FUniqueNetId& PlayerId)
{
#if WITH_ENGINE
	IOnlineVoicePtr VoiceInt = NullSubsystem->GetVoiceInterface();
	if (VoiceInt.IsValid())
	{
		if (!NullSubsystem->IsLocalPlayer(PlayerId))
		{
			VoiceInt->RegisterRemoteTalker(PlayerId);
		}
		else
		{
			// This is a local player. In case their PlayerState came last during replication, reprocess muting
			VoiceInt->ProcessMuteChangeNotification();
		}
	}
#endif //WITH_ENGINE
}

void FOnlineSessionNull::UnregisterVoice(const FUniqueNetId& PlayerId)
{
#if WITH_ENGINE
	IOnlineVoicePtr VoiceInt = NullSubsystem->GetVoiceInterface();
	if (VoiceInt.IsValid())
	{
		if (!NullSubsystem->IsLocalPlayer(PlayerId))
		{
			if (VoiceInt.IsValid())
			{
				VoiceInt->UnregisterRemoteTalker(PlayerId);
			}
		}
	}
#endif //WITH_ENGINE
}

bool FOnlineSessionNull::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

bool FOnlineSessionNull::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccess = false;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		bSuccess = true;

		for (int32 PlayerIdx=0; PlayerIdx<Players.Num(); PlayerIdx++)
		{
			const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

			FUniqueNetIdMatcher PlayerMatch(*PlayerId);
			if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
			{
				Session->RegisteredPlayers.Add(PlayerId);
				RegisterVoice(*PlayerId);

				// update number of open connections
				if (Session->NumOpenPublicConnections > 0)
				{
					Session->NumOpenPublicConnections--;
				}
				else if (Session->NumOpenPrivateConnections > 0)
				{
					Session->NumOpenPrivateConnections--;
				}
			}
			else
			{
				RegisterVoice(*PlayerId);
				UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *PlayerId->ToDebugString(), *SessionName.ToString());
			}			
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
	}

	TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

bool FOnlineSessionNull::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(PlayerId.AsShared());
	return UnregisterPlayers(SessionName, Players);
}

bool FOnlineSessionNull::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccess = true;

	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
		{
			const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

			FUniqueNetIdMatcher PlayerMatch(*PlayerId);
			int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
			if (RegistrantIndex != INDEX_NONE)
			{
				Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
				UnregisterVoice(*PlayerId);

				// update number of open connections
				if (Session->NumOpenPublicConnections < Session->SessionSettings.NumPublicConnections)
				{
					Session->NumOpenPublicConnections++;
				}
				else if (Session->NumOpenPrivateConnections < Session->SessionSettings.NumPrivateConnections)
				{
					Session->NumOpenPrivateConnections++;
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Player %s is not part of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
		bSuccess = false;
	}

	TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

void FOnlineSessionNull::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_Session_Interface);
	TickLanTasks(DeltaTime);
}

void FOnlineSessionNull::TickLanTasks(float DeltaTime)
{
	LANSessionManager.Tick(DeltaTime);
}

void FOnlineSessionNull::AppendSessionToPacket(FNboSerializeToBufferNull& Packet, FOnlineSession* Session)
{
	/** Owner of the session */
	((FNboSerializeToBuffer&) Packet) << Session->OwningUserId->ToString()
		<< Session->OwningUserName
		<< Session->NumOpenPrivateConnections
		<< Session->NumOpenPublicConnections;

	// Try to get the actual port the netdriver is using
	SetPortFromNetDriver(*NullSubsystem, Session->SessionInfo);

	// Write host info (host addr, session id, and key)
	Packet << *StaticCastSharedPtr<FOnlineSessionInfoNull>(Session->SessionInfo);

	// Now append per game settings
	AppendSessionSettingsToPacket(Packet, &Session->SessionSettings);
}

void FOnlineSessionNull::AppendSessionSettingsToPacket(FNboSerializeToBufferNull& Packet, FOnlineSessionSettings* SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Sending session settings to client"));
#endif 

	// Members of the session settings class
	((FNboSerializeToBuffer&)Packet) << SessionSettings->NumPublicConnections
		<< SessionSettings->NumPrivateConnections
		<< (uint8)SessionSettings->bShouldAdvertise
		<< (uint8)SessionSettings->bIsLANMatch
		<< (uint8)SessionSettings->bIsDedicated
		<< (uint8)SessionSettings->bUsesStats
		<< (uint8)SessionSettings->bAllowJoinInProgress
		<< (uint8)SessionSettings->bAllowInvites
		<< (uint8)SessionSettings->bUsesPresence
		<< (uint8)SessionSettings->bAllowJoinViaPresence
		<< (uint8)SessionSettings->bAllowJoinViaPresenceFriendsOnly
		<< (uint8)SessionSettings->bAntiCheatProtected
	    << SessionSettings->BuildUniqueId;

	// First count number of advertised keys
	int32 NumAdvertisedProperties = 0;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{	
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			NumAdvertisedProperties++;
		}
	}

	// Add count of advertised keys and the data
	((FNboSerializeToBuffer&)Packet) << (int32)NumAdvertisedProperties;
	for (FSessionSettings::TConstIterator It(SessionSettings->Settings); It; ++It)
	{
		const FOnlineSessionSetting& Setting = It.Value();
		if (Setting.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			((FNboSerializeToBuffer&)Packet) << It.Key();
			Packet << Setting;
#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting.ToString());
#endif
		}
	}
}

void FOnlineSessionNull::OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce)
{
	// Iterate through all registered sessions and respond for each one that can be joinable
	UE::TScopeLock ScopeLock(SessionLock);
	for (int32 SessionIndex = 0; SessionIndex < Sessions.Num(); SessionIndex++)
	{
		FNamedOnlineSession* Session = &Sessions[SessionIndex];

		// Don't respond to query if the session is not a joinable LAN match.
		if (Session && IsSessionJoinable(*Session))
		{
			FNboSerializeToBufferNull Packet(LAN_BEACON_MAX_PACKET_SIZE);
			// Create the basic header before appending additional information
			LANSessionManager.CreateHostResponsePacket(Packet, ClientNonce);

			// Add all the session details
			AppendSessionToPacket(Packet, Session);

			// Broadcast this response so the client can see us
			if (!Packet.HasOverflow())
			{
				LANSessionManager.BroadcastPacket(Packet, Packet.GetByteCount());
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("LAN broadcast packet overflow, cannot broadcast on LAN"));
			}
		}
	}
}

void FOnlineSessionNull::ReadSessionFromPacket(FNboSerializeFromBufferNull& Packet, FOnlineSession* Session)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading session information from server"));
#endif

	/** Owner of the session */
	FUniqueNetIdNullRef OwningUserId = FUniqueNetIdNull::Create();
	Packet >> const_cast<FUniqueNetIdNull&>(*OwningUserId)
		>> Session->OwningUserName
		>> Session->NumOpenPrivateConnections
		>> Session->NumOpenPublicConnections;

	Session->OwningUserId = OwningUserId;

	// Allocate and read the connection data
	FOnlineSessionInfoNull* NullSessionInfo = new FOnlineSessionInfoNull();
	NullSessionInfo->HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	Packet >> *NullSessionInfo;
	Session->SessionInfo = MakeShareable(NullSessionInfo); 

	// Read any per object data using the server object
	ReadSettingsFromPacket(Packet, Session->SessionSettings);
}

void FOnlineSessionNull::ReadSettingsFromPacket(FNboSerializeFromBufferNull& Packet, FOnlineSessionSettings& SessionSettings)
{
#if DEBUG_LAN_BEACON
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Reading game settings from server"));
#endif

	// Clear out any old settings
	SessionSettings.Settings.Empty();

	// Members of the session settings class
	Packet >> SessionSettings.NumPublicConnections
		>> SessionSettings.NumPrivateConnections;
	uint8 Read = 0;
	// Read all the bools as bytes
	Packet >> Read;
	SessionSettings.bShouldAdvertise = !!Read;
	Packet >> Read;
	SessionSettings.bIsLANMatch = !!Read;
	Packet >> Read;
	SessionSettings.bIsDedicated = !!Read;
	Packet >> Read;
	SessionSettings.bUsesStats = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinInProgress = !!Read;
	Packet >> Read;
	SessionSettings.bAllowInvites = !!Read;
	Packet >> Read;
	SessionSettings.bUsesPresence = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresence = !!Read;
	Packet >> Read;
	SessionSettings.bAllowJoinViaPresenceFriendsOnly = !!Read;
	Packet >> Read;
	SessionSettings.bAntiCheatProtected = !!Read;

	// BuildId
	Packet >> SessionSettings.BuildUniqueId;

	// Now read the contexts and properties from the settings class
	int32 NumAdvertisedProperties = 0;
	// First, read the number of advertised properties involved, so we can presize the array
	Packet >> NumAdvertisedProperties;
	if (Packet.HasOverflow() == false)
	{
		FName Key;
		// Now read each context individually
		for (int32 Index = 0;
			Index < NumAdvertisedProperties && Packet.HasOverflow() == false;
			Index++)
		{
			FOnlineSessionSetting Setting;
			Packet >> Key;
			Packet >> Setting;
			SessionSettings.Set(Key, Setting);

#if DEBUG_LAN_BEACON
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *Setting->ToString());
#endif
		}
	}
	
	// If there was an overflow, treat the string settings/properties as broken
	if (Packet.HasOverflow())
	{
		SessionSettings.Settings.Empty();
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Packet overflow detected in ReadGameSettingsFromPacket()"));
	}
}

void FOnlineSessionNull::OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength)
{
	if (CurrentSessionSearch.IsValid())
	{
		FOnlineSessionSearchResult SearchResult;

		// this is not a correct ping, but better than nothing
		SearchResult.PingInMs = static_cast<int32>((FPlatformTime::Seconds() - SessionSearchStartInSeconds) * 1000);

		// Prepare to read data from the packet
		FNboSerializeFromBufferNull Packet(PacketData, PacketLength);

		ReadSessionFromPacket(Packet, &SearchResult.Session);

		const int32 BuildUniqueId = GetBuildUniqueId();

		if (SearchResult.Session.SessionSettings.BuildUniqueId == BuildUniqueId)
		{
			// Deduplicate by SessionId so that retried queries don't produce duplicate search results, while still allowing 
			// multiple different sessions from the same host to appear 
			const FString& NewSessionId = SearchResult.Session.GetSessionIdStr();
			bool bAlreadyInResults = false;
			for (const FOnlineSessionSearchResult& Existing : CurrentSessionSearch->SearchResults)
			{
				if (Existing.Session.GetSessionIdStr() == NewSessionId)
				{
					bAlreadyInResults = true;
					break;
				}
			}

			if (!bAlreadyInResults)
			{
				CurrentSessionSearch->SearchResults.Emplace(MoveTemp(SearchResult));
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Rejecting search result [%s]: mismatched build id. Local: %d - Session: %d"),
				*SearchResult.Session.GetSessionIdStr(),
				BuildUniqueId,
				SearchResult.Session.SessionSettings.BuildUniqueId);
		}
		// NOTE: we don't notify until the timeout happens
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create new online game settings object"));
	}
}

uint32 FOnlineSessionNull::FinalizeLANSearch()
{
	if (LANSessionManager.GetBeaconState() == ELanBeaconState::Searching)
	{
		LANSessionManager.StopLANSession();
	}

	return UpdateLANStatus();
}

void FOnlineSessionNull::OnLANSearchTimeout()
{
	FinalizeLANSearch();

	if (CurrentSessionSearch.IsValid())
	{
		if (CurrentSessionSearch->SearchResults.Num() > 0)
		{
			// Allow game code to sort the servers
			CurrentSessionSearch->SortSearchResults();
		}
		CurrentSessionSearch->SearchState = EOnlineAsyncTaskState::Done;

		CurrentSessionSearch = NULL;
	}

	// Trigger the delegate as complete
	TriggerOnFindSessionsCompleteDelegates(true);
}

int32 FOnlineSessionNull::GetNumSessions()
{
	UE::TScopeLock ScopeLock(SessionLock);
	return Sessions.Num();
}

void FOnlineSessionNull::DumpSessionState()
{
	UE::TScopeLock ScopeLock(SessionLock);

	for (int32 SessionIdx=0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		DumpNamedSession(&Sessions[SessionIdx]);
	}
}

void FOnlineSessionNull::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, EOnJoinSessionCompleteResult::Success);
}

void FOnlineSessionNull::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, true);
}

void FOnlineSessionNull::SetPortFromNetDriver(const FOnlineSubsystemNull& Subsystem, const TSharedPtr<FOnlineSessionInfo>& SessionInfo)
{
#if WITH_ENGINE
	auto NetDriverPort = GetPortFromNetDriver(Subsystem.GetInstanceName());
	auto SessionInfoNull = StaticCastSharedPtr<FOnlineSessionInfoNull>(SessionInfo);
	if (SessionInfoNull.IsValid() && SessionInfoNull->HostAddr.IsValid())
	{
		SessionInfoNull->HostAddr->SetPort(NetDriverPort);
	}
#endif //WITH_ENGINE
}

bool FOnlineSessionNull::IsHost(const FNamedOnlineSession& Session) const
{
	if (NullSubsystem->IsDedicated())
	{
		return true;
	}

	IOnlineIdentityPtr IdentityInt = NullSubsystem->GetIdentityInterface(); 
	if (!IdentityInt.IsValid())
	{
		return false;
	}

	FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(Session.HostingPlayerNum);
	return (UserId.IsValid() && (*UserId == *Session.OwningUserId));
}

FUniqueNetIdPtr FOnlineSessionNull::CreateSessionIdFromString(const FString& SessionIdStr)
{
	FUniqueNetIdPtr SessionId;
	if (!SessionIdStr.IsEmpty())
	{
		SessionId = FUniqueNetIdNull::Create(SessionIdStr);
	}
	return SessionId;
}
