// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionDelegates.h"
#include "OnlineSessionSettings.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemNullPackage.h"
#include "Online/LANBeacon.h"

class FOnlineSubsystemNull;

/**
 * Interface definition for the online services session services 
 * Session services are defined as anything related managing a session 
 * and its state within a platform service
 */
class FOnlineSessionNull : public IOnlineSession
{
private:

	/** Reference to the main Null subsystem */
	class FOnlineSubsystemNull* NullSubsystem;

	/** Handles advertising sessions over LAN and client searches */
	FLANSession LANSessionManager;

	/** Hidden on purpose */
	FOnlineSessionNull() :
		NullSubsystem(NULL),
		CurrentSessionSearch(NULL)
	{}

	/**
	 * Ticks any lan beacon background tasks
	 *
	 * @param DeltaTime the time since the last tick
	 */
	void TickLanTasks(float DeltaTime);

	/**
	 * Checks whether there are any sessions that need to be advertised (over LAN)
	 *
	 * @return true if there is at least one
	 */
	bool NeedsToAdvertise();

	/**
	 * Determines whether this particular session should be advertised (over LAN)
	 *
	 * @return true if yes
	 */
	bool NeedsToAdvertise( FNamedOnlineSession& Session );

	/**
	 * Determines whether this particular session is joinable.
	 *
	 * @return true if yes
	 */
	bool IsSessionJoinable( const FNamedOnlineSession& Session) const;

	/**
	 * Updates the status of LAN session (creates it if needed, shuts down if not)
	 * 
	 * @return ONLINE_SUCCESS if everything went successful, an error code otherwise
	 */
	uint32 UpdateLANStatus();

	/**
	 *	Join a LAN session
	 * 
	 * @param PlayerNum local index of the user initiating the request
	 * @param Session newly allocated session with join information
	 * @param SearchSession the desired session to join
	 * 
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 JoinLANSession(int32 PlayerNum, class FNamedOnlineSession* Session, const class FOnlineSession* SearchSession);

	/**
	 * Builds a LAN search query and broadcasts it
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 FindLANSession();

	/**
	 * Finishes searching over LAN and returns to hosting (if needed)
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 FinalizeLANSearch();

	/**
	 * Adds the game session data to the packet that is sent by the host
	 * in response to a server query
	 *
	 * @param Packet the writer object that will encode the data
	 * @param Session the session to add to the packet
	 */
	void AppendSessionToPacket(class FNboSerializeToBufferNull& Packet, class FOnlineSession* Session);

	/**
	 * Adds the game settings data to the packet that is sent by the host
	 * in response to a server query
	 *
	 * @param Packet the writer object that will encode the data
	 * @param SessionSettings the session settings to add to the packet
	 */
	void AppendSessionSettingsToPacket(class FNboSerializeToBufferNull& Packet, FOnlineSessionSettings* SessionSettings);

	/**
	 * Reads the settings data from the packet and applies it to the
	 * specified object
	 *
	 * @param Packet the reader object that will read the data
	 * @param SessionSettings the session settings to copy the data to
	 */
	void ReadSessionFromPacket(class FNboSerializeFromBufferNull& Packet, class FOnlineSession* Session);

	/**
	 * Reads the settings data from the packet and applies it to the
	 * specified object
	 *
	 * @param Packet the reader object that will read the data
	 * @param SessionSettings the session settings to copy the data to
	 */
	void ReadSettingsFromPacket(class FNboSerializeFromBufferNull& Packet, FOnlineSessionSettings& SessionSettings);

	/**
	 * Delegate triggered when the LAN beacon has detected a valid client request has been received
	 *
	 * @param PacketData packet data sent by the requesting client with header information removed
	 * @param PacketLength length of the packet not including header size
	 * @param ClientNonce the nonce returned by the client to return with the server packet
	 */
	void OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce);

	/**
	 * Delegate triggered when the LAN beacon has detected a valid host response to a client request has been received
	 *
	 * @param PacketData packet data sent by the requesting client with header information removed
	 * @param PacketLength length of the packet not including header size
	 */
	void OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength);

	/**
	 * Delegate triggered when the LAN beacon has finished searching (some time after last received host packet)
	 */
	void OnLANSearchTimeout();

	/**
	 * Attempt to set the host port in the session info based on the actual port the netdriver is using.
	 */
	static void SetPortFromNetDriver(const FOnlineSubsystemNull& Subsystem, const TSharedPtr<FOnlineSessionInfo>& SessionInfo);

	/**
	 * Returns true if the session owner is also the host.
	 */
	bool IsHost(const FNamedOnlineSession& Session) const;

PACKAGE_SCOPE:

	/** Critical sections for thread safe operation of session lists */
	mutable FTransactionallySafeCriticalSection SessionLock;

	/** Current session settings */
	TArray<FNamedOnlineSession> Sessions;

	/** Current search object */
	TSharedPtr<FOnlineSessionSearch> CurrentSessionSearch;

	/** Current search start time. */
	double SessionSearchStartInSeconds;

	FOnlineSessionNull(class FOnlineSubsystemNull* InSubsystem) :
		NullSubsystem(InSubsystem),
		CurrentSessionSearch(NULL),
		SessionSearchStartInSeconds(0)
	{}

	/**
	 * Session tick for various background tasks
	 */
	void Tick(float DeltaTime);

	// IOnlineSession
	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		return &Sessions.Emplace_GetRef(SessionName, SessionSettings);
	}

	class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		return &Sessions.Emplace_GetRef(SessionName, Session);
	}

	/**
	 * Parse the command line for invite/join information at launch
	 */
	void CheckPendingSessionInvite();

	/**
	* Registers and updates voice data for the given player id
	*
	* @param PlayerId player to register with the voice subsystem
	*/
	void RegisterVoice(const FUniqueNetId& PlayerId);

	/**
	* Unregisters a given player id from the voice subsystem
	*
	* @param PlayerId player to unregister with the voice subsystem
	*/
	void UnregisterVoice(const FUniqueNetId& PlayerId);

	/**
	 * Registers all local players with the current session
	 *
	 * @param Session the session that they are registering in
	 */
	void RegisterLocalPlayers(class FNamedOnlineSession* Session);

public:

	virtual ~FOnlineSessionNull() = default;

	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;

	FNamedOnlineSession* GetNamedSession(FName SessionName) override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return &Sessions[SearchIndex];
			}
		}
		return NULL;
	}

	virtual void RemoveNamedSession(FName SessionName) override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				Sessions.RemoveAtSwap(SearchIndex);
				return;
			}
		}
	}

	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionName == SessionName)
			{
				return Sessions[SearchIndex].SessionState;
			}
		}

		return EOnlineSessionState::NoSession;
	}

	virtual bool HasPresenceSession() override
	{
		UE::TScopeLock ScopeLock(SessionLock);
		for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
		{
			if (Sessions[SearchIndex].SessionSettings.bUsesPresence)
			{
				return true;
			}
		}
		
		return false;
	}

	// IOnlineSession
	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool StartSession(FName SessionName) override;
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = true) override;
	virtual bool EndSession(FName SessionName) override;
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;
	virtual bool CancelFindSessions() override;
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;
	virtual bool JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) override;
	virtual bool GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) override;
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) override;
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) override;
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) override;
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) override;
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual int32 GetNumSessions() override;
	virtual void DumpSessionState() override;
};

typedef TSharedPtr<FOnlineSessionNull, ESPMode::ThreadSafe> FOnlineSessionNullPtr;
