// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStoreV2InterfaceNull.h"
#include "OnlineSubsystemNull.h"
#include "OnlineSubsystemNullTypes.h"
#include "OnlineError.h"

FOnlineStoreV2Null::FOnlineStoreV2Null(FOnlineSubsystemNull& InNullSubsystem)
	: NullSubsystem(InNullSubsystem)
{
	CreateFakeOffer(TEXT("Item1_Id"), TEXT("Cool Item1"), TEXT("Super cool Item1"), 3);
	CreateFakeOffer(TEXT("Item2_Id"), TEXT("Nice Item2"), TEXT("Very nice Item2"), 40);
	CreateFakeOffer(TEXT("Item3_Id"), TEXT("Fab Item3"), TEXT("Faboulous Item3"), 500);
	CreateFakeOffer(TEXT("Item4_Id"), TEXT("$$$ Item4"), TEXT("Expensive Item4"), 6000);
	CreateFakeOffer(TEXT("Item5_Id"), TEXT("Fake Item5"), TEXT("Sooo fake Item5"), 70000);
}

void FOnlineStoreV2Null::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	NullSubsystem.ExecuteNextTick([Delegate]()
	{
		Delegate.ExecuteIfBound(false, TEXT("FOnlineStoreV2Null::QueryCategories Not Implemented"));
	});
}

void FOnlineStoreV2Null::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	OutCategories.Empty();
}

void FOnlineStoreV2Null::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	if (!UserId.IsValid())
	{
		NullSubsystem.ExecuteNextTick([Delegate]()
		{
			const bool bWasSuccessful = false;
			Delegate.ExecuteIfBound(bWasSuccessful, TArray<FUniqueOfferId>(), FString(TEXT("FOnlineStoreV2Null::QueryOffersByFilter User invalid")));
		});
		return;
	}

	QueryOffers(static_cast<const FUniqueNetIdNull&>(UserId), TArray<FUniqueOfferId>(), Delegate);
}

void FOnlineStoreV2Null::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	if (!UserId.IsValid())
	{
		NullSubsystem.ExecuteNextTick([Delegate]()
		{
			const bool bWasSuccessful = false;
			Delegate.ExecuteIfBound(bWasSuccessful, TArray<FUniqueOfferId>(), FString(TEXT("FOnlineStoreV2Null::QueryOffersByFilter User invalid")));
		});
		return;
	}

	if (OfferIds.Num() < 1)
	{
		NullSubsystem.ExecuteNextTick([Delegate]()
		{
			constexpr bool bWasSuccessful = false;
			Delegate.ExecuteIfBound(bWasSuccessful, TArray<FUniqueOfferId>(), TEXT("FOnlineStoreV2Null::No OfferIds requested"));
		});
		return;
	}

	QueryOffers(static_cast<const FUniqueNetIdNull&>(UserId), OfferIds, Delegate);
}

void FOnlineStoreV2Null::QueryOffers(const FUniqueNetIdNull& NullUserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	TWeakPtr<FOnlineStoreV2Null, ESPMode::ThreadSafe> WeakMe = AsShared();

	NullSubsystem.ExecuteNextTick([WeakMe, Delegate]()
	{
		FOnlineStoreV2NullPtr StrongThis = WeakMe.Pin();
		if(StrongThis.IsValid())
		{
			FOnlineError ResultStatus(true);

			TArray<FUniqueOfferId> FoundOffersData;
			StrongThis->AvailableOffers.GenerateKeyArray(FoundOffersData);

			Delegate.ExecuteIfBound(ResultStatus.bSucceeded, FoundOffersData, ResultStatus.ErrorMessage.ToString());
		}
	});
}

void FOnlineStoreV2Null::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	AvailableOffers.GenerateValueArray(OutOffers);
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreV2Null::GetOffer(const FUniqueOfferId& OfferId) const
{
	const FOnlineStoreOfferRef* const FoundOfferPtr = AvailableOffers.Find(OfferId);
	if (FoundOfferPtr == nullptr)
	{
		return nullptr;
	}

	return *FoundOfferPtr;
}

void FOnlineStoreV2Null::CreateFakeOffer(const FString& Id, const FString& Title, const FString& Description, int32 Price)
{
	TSharedRef<FOnlineStoreOffer> Offer = MakeShared<FOnlineStoreOffer>();
	Offer->OfferId = Id;
	Offer->Title = FText::FromString(Title);
	Offer->Description = FText::FromString(Description);
	Offer->NumericPrice = Price;
	Offer->RegularPrice = Price;
	Offer->CurrencyCode = TEXT("USD");

	AvailableOffers.Add(Id, Offer);
}
