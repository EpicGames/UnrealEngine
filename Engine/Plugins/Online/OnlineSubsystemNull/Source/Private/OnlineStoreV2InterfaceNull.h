// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlineSubsystemNullPackage.h"

class FOnlineSubsystemNull;
class FUniqueNetIdNull;

/**
 * Implementation for online store via Null interface
 */
class FOnlineStoreV2Null : public IOnlineStoreV2, public TSharedFromThis<FOnlineStoreV2Null, ESPMode::ThreadSafe>
{
public:
	FOnlineStoreV2Null(FOnlineSubsystemNull& InNullSubsystem);
	virtual ~FOnlineStoreV2Null() = default;

public:// IOnlineStoreV2
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;

PACKAGE_SCOPE:
	void QueryOffers(const FUniqueNetIdNull& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate);

PACKAGE_SCOPE:
	FOnlineSubsystemNull& NullSubsystem;
	TMap<FUniqueOfferId, FOnlineStoreOfferRef> AvailableOffers;

private:
	void CreateFakeOffer(const FString& Id, const FString& Title, const FString& Description, int32 Price);
};

using FOnlineStoreNullPtr = TSharedPtr<FOnlineStoreV2Null, ESPMode::ThreadSafe>;
using FOnlineStoreNullRef = TSharedRef<FOnlineStoreV2Null, ESPMode::ThreadSafe>;
