// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemNullModule.h"
#include "OnlineSubsystemNull.h"

IMPLEMENT_MODULE(FOnlineSubsystemNullModule, OnlineSubsystemNull);

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryNull : public IOnlineFactory
{
public:

	FOnlineFactoryNull() {}
	virtual ~FOnlineFactoryNull() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemNullPtr OnlineSub = MakeShared<FOnlineSubsystemNull, ESPMode::ThreadSafe>(InstanceName);
		if (OnlineSub->IsEnabled())
		{
			if(!OnlineSub->Init())
			{
				UE_LOG_ONLINE(Warning, TEXT("Null API failed to initialize!"));
				OnlineSub->Shutdown();
				OnlineSub = NULL;
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Null API disabled!"));
			OnlineSub->Shutdown();
			OnlineSub = NULL;
		}

		return OnlineSub;
	}
};

void FOnlineSubsystemNullModule::StartupModule()
{
	NullFactory = new FOnlineFactoryNull();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(NULL_SUBSYSTEM, NullFactory);
}

void FOnlineSubsystemNullModule::ShutdownModule()
{
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(NULL_SUBSYSTEM);
	
	delete NullFactory;
	NullFactory = NULL;
}
