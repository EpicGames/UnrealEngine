// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemNull.h"

#include "HAL/RunnableThread.h"
#include "Misc/Fork.h"
#include "Misc/FileHelper.h"

#include "OnlineAchievementsInterfaceNull.h"
#include "OnlineAsyncTaskManagerNull.h"
#include "OnlineExternalUINull.h"
#include "OnlineIdentityNull.h"
#include "OnlineLeaderboardInterfaceNull.h"
#include "OnlineMessageSanitizerNull.h"
#include "OnlinePurchaseInterfaceNull.h"
#include "OnlineSessionInterfaceNull.h"
#include "OnlineStoreV2InterfaceNull.h"
#include "OnlineTitleFileNull.h"

#if WITH_ENGINE
#include "VoiceInterfaceImpl.h"
#endif

#include "Misc/CommandLine.h"

#if WITH_ENGINE
#endif //WITH_ENGINE

FThreadSafeCounter FOnlineSubsystemNull::TaskCounter;

IOnlineSessionPtr FOnlineSubsystemNull::GetSessionInterface() const
{
	return SessionInterface;
}

IOnlineFriendsPtr FOnlineSubsystemNull::GetFriendsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemNull::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemNull::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemNull::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemNull::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemNull::GetEntitlementsInterface() const
{
	return nullptr;
};

IOnlineLeaderboardsPtr FOnlineSubsystemNull::GetLeaderboardsInterface() const
{
	return LeaderboardsInterface;
}

IOnlineVoicePtr FOnlineSubsystemNull::GetVoiceInterface() const
{
#if WITH_ENGINE
	if (VoiceInterface.IsValid() && !bVoiceInterfaceInitialized)
	{	
		if (!VoiceInterface->Init())
		{
			VoiceInterface = nullptr;
		}

		bVoiceInterfaceInitialized = true;
	}

	return VoiceInterface;
#else //WITH_ENGINE
	return nullptr;
#endif //WITH_ENGINE
}

IOnlineExternalUIPtr FOnlineSubsystemNull::GetExternalUIInterface() const
{
	// This may be null depending on cvars
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemNull::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemNull::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineTitleFilePtr FOnlineSubsystemNull::GetTitleFileInterface() const
{
	return TitleFileInterface;
}

IOnlineStoreV2Ptr FOnlineSubsystemNull::GetStoreV2Interface() const
{
	return StoreV2Interface;
}

IOnlinePurchasePtr FOnlineSubsystemNull::GetPurchaseInterface() const
{
	return PurchaseInterface;
}

IOnlineEventsPtr FOnlineSubsystemNull::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemNull::GetAchievementsInterface() const
{
	return AchievementsInterface;
}

IOnlineSharingPtr FOnlineSubsystemNull::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemNull::GetUserInterface() const
{
	return nullptr;
}

IOnlineMessagePtr FOnlineSubsystemNull::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemNull::GetPresenceInterface() const
{
	return nullptr;
}

IOnlineChatPtr FOnlineSubsystemNull::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemNull::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemNull::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemNull::GetTournamentInterface() const
{
	return nullptr;
}

IMessageSanitizerPtr FOnlineSubsystemNull::GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const
{
	return MessageSanitizerInterface;
}

bool FOnlineSubsystemNull::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemNull_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		OnlineAsyncTaskThreadRunnable->GameTick();
	}

	if (SessionInterface.IsValid())
	{
		SessionInterface->Tick(DeltaTime);
	}

	if (TitleFileInterface.IsValid())
	{
		TitleFileInterface->Tick(DeltaTime);
	}

#if WITH_ENGINE
	if (VoiceInterface.IsValid() && bVoiceInterfaceInitialized)
	{
		VoiceInterface->Tick(DeltaTime);
	}
#endif //WITH_ENGINE

	return true;
}

// Declares console variables that modify how OSS null works so it can emulate different platforms for testing. These are also loaded from [OnlineSubsystemNull]

bool FOnlineSubsystemNull::bAutoLoginAtStartup = true;
FAutoConsoleVariableRef CVarAutoLoginAtStartup(
	TEXT("OSSNull.AutoLoginAtStartup"),
	FOnlineSubsystemNull::bAutoLoginAtStartup,
	TEXT("True if it should login the first user at startup like single-user platforms, false to only login when requested"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bSupportExternalUI = false;
FAutoConsoleVariableRef CVarSupportExternalUI(
	TEXT("OSSNull.SupportExternalUI"),
	FOnlineSubsystemNull::bSupportExternalUI,
	TEXT("True if it should support an external UI interface"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bRequireShowLoginUI = false;
FAutoConsoleVariableRef CVarRequireShowLoginUI(
	TEXT("OSSNull.RequireShowLoginUI"),
	FOnlineSubsystemNull::bRequireShowLoginUI,
	TEXT("True if login requires calling ShowLoginUI on the externalUI, depends on SupportExternalUI"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bForceShowLoginUIUserChange = false;
FAutoConsoleVariableRef CVarForceShowLoginUIUserChange(
	TEXT("OSSNull.ForceShowLoginUIUserChange"),
	FOnlineSubsystemNull::bForceShowLoginUIUserChange,
	TEXT("True if the user index should change during login UI to emulate a platform user change"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bRequireLoginCredentials = false;
FAutoConsoleVariableRef CVarRequireLoginCredentials(
	TEXT("OSSNull.RequireLoginCredentials"),
	FOnlineSubsystemNull::bRequireLoginCredentials,
	TEXT("True if login should require a user/pass to act like an external service, false to match most platforms and use the default"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bAddUserNumToNullId = false;
FAutoConsoleVariableRef CVarAddUserNumToNullId(
	TEXT("OSSNull.AddUserNumToNullId"),
	FOnlineSubsystemNull::bAddUserNumToNullId,
	TEXT("True if login name should include the local user number, which allows different stable Ids per user num"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bForceStableNullId = false;
FAutoConsoleVariableRef CVarForceStableNullId(
	TEXT("OSSNull.ForceStableNullId"),
	FOnlineSubsystemNull::bForceStableNullId,
	TEXT("True if it should use a system-stable null Id for login, same as -StableNullID on command line"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bForceOfflineMode = false;
FAutoConsoleVariableRef CVarForceOfflineMode(
	TEXT("OSSNull.ForceOfflineMode"),
	FOnlineSubsystemNull::bForceOfflineMode,
	TEXT("True if it should fail faked network queries and act like an offline system"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::bOnlineRequiresSecondLogin = false;
FAutoConsoleVariableRef CVarOnlineRequiresSecondLogin(
	TEXT("OSSNull.OnlineRequiresSecondLogin"),
	FOnlineSubsystemNull::bOnlineRequiresSecondLogin,
	TEXT("True if the first login only counts as local login, a second is required for online access"),
	ECVF_Default | ECVF_Cheat);

bool FOnlineSubsystemNull::Init()
{
	const bool bNullInit = true;
	if (bNullInit)
	{
		static bool bLoadedConfig = false;
		if (!bLoadedConfig)
		{
#if WITH_ENGINE
			// Read configuration variables once for emulating other login systems, cvars can override in PIE
			bLoadedConfig = true;
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bAutoLoginAtStartup"), bAutoLoginAtStartup, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bSupportExternalUI"), bSupportExternalUI, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bRequireShowLoginUI"), bRequireShowLoginUI, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bForceShowLoginUIUserChange"), bForceShowLoginUIUserChange, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bRequireLoginCredentials"), bRequireLoginCredentials, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bAddUserNumToNullId"), bAddUserNumToNullId, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bForceStableNullId"), bForceStableNullId, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bForceOfflineMode"), bForceOfflineMode, GEngineIni);
			GConfig->GetBool(TEXT("OnlineSubsystemNull"), TEXT("bOnlineRequiresSecondLogin"), bOnlineRequiresSecondLogin, GEngineIni);

			if (FParse::Param(FCommandLine::Get(), TEXT("StableNullID")))
			{
				bForceStableNullId = true;
			}
#endif
		}

		// Create the online async task thread
		OnlineAsyncTaskThreadRunnable = new FOnlineAsyncTaskManagerNull(this);
		check(OnlineAsyncTaskThreadRunnable);
		OnlineAsyncTaskThread = FForkProcessHelper::CreateForkableThread(OnlineAsyncTaskThreadRunnable, *FString::Printf(TEXT("OnlineAsyncTaskThreadNull %s(%d)"), *InstanceName.ToString(), TaskCounter.Increment()), 128 * 1024, TPri_Normal);
		check(OnlineAsyncTaskThread);
		UE_LOG_ONLINE(Verbose, TEXT("Created thread (ID:%d)."), OnlineAsyncTaskThread->GetThreadID());

		SessionInterface = MakeShareable(new FOnlineSessionNull(this));
		LeaderboardsInterface = MakeShareable(new FOnlineLeaderboardsNull(this));
		IdentityInterface = MakeShareable(new FOnlineIdentityNull(this));
		if (bSupportExternalUI)
		{
			ExternalUIInterface = MakeShareable(new FOnlineExternalUINull(this));
		}
		AchievementsInterface = MakeShareable(new FOnlineAchievementsNull(this));
#if WITH_ENGINE
		VoiceInterface = MakeShareable(new FOnlineVoiceImpl(this));
#endif //WITH_ENGINE
		StoreV2Interface = MakeShareable(new FOnlineStoreV2Null(*this));
		PurchaseInterface = MakeShareable(new FOnlinePurchaseNull(*this));
		MessageSanitizerInterface = MakeShareable(new FMessageSanitizerNull());
		TitleFileInterface = MakeShareable(new FOnlineTitleFileNull());
	}
	else
	{
		Shutdown();
	}

	return bNullInit;
}

bool FOnlineSubsystemNull::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemNull::Shutdown()"));

	FOnlineSubsystemImpl::Shutdown();

	if (OnlineAsyncTaskThread)
	{
		// Destroy the online async task thread
		delete OnlineAsyncTaskThread;
		OnlineAsyncTaskThread = nullptr;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		delete OnlineAsyncTaskThreadRunnable;
		OnlineAsyncTaskThreadRunnable = nullptr;
	}

#if WITH_ENGINE
	if (VoiceInterface.IsValid() && bVoiceInterfaceInitialized)
	{
		VoiceInterface->Shutdown();
	}
#endif //WITH_ENGINE
	
#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}
 
	// Destruct the interfaces
	DESTRUCT_INTERFACE(PurchaseInterface);
	DESTRUCT_INTERFACE(StoreV2Interface);
	DESTRUCT_INTERFACE(VoiceInterface);
	DESTRUCT_INTERFACE(AchievementsInterface);
	DESTRUCT_INTERFACE(IdentityInterface);
	DESTRUCT_INTERFACE(LeaderboardsInterface);
	DESTRUCT_INTERFACE(SessionInterface);
	DESTRUCT_INTERFACE(MessageSanitizerInterface);
	DESTRUCT_INTERFACE(TitleFileInterface);

#undef DESTRUCT_INTERFACE
	
	return true;
}

FString FOnlineSubsystemNull::GetAppId() const
{
	return TEXT("");
}

bool FOnlineSubsystemNull::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	return false;
}
FText FOnlineSubsystemNull::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemNull", "OnlineServiceName", "Null");
}

