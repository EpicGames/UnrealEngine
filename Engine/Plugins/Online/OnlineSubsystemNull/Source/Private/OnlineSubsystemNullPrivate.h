// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"

#define INVALID_INDEX -1

/** URL Prefix when using Null socket connection */
#define NULL_URL_PREFIX TEXT("Null.")

/** pre-pended to all NULL logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("NULL: ")


