// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "IPAddress.h"
#include "OnlineSubsystemPackage.h"

class FOnlineSubsystemNull;

// from OnlineSubsystemTypes.h
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdNull, NULL_SUBSYSTEM);

/** 
 * Implementation of session information
 */
class FOnlineSessionInfoNull : public FOnlineSessionInfo
{
	/** Hidden on purpose */
	FOnlineSessionInfoNull(const FOnlineSessionInfoNull& Src) = delete;
	FOnlineSessionInfoNull& operator=(const FOnlineSessionInfoNull& Src) = delete;

PACKAGE_SCOPE:

	/** Constructor */
	FOnlineSessionInfoNull();

	/** 
	 * Initialize a Null session info with the address of this machine
	 * and an id for the session
	 */
	void Init(const FOnlineSubsystemNull& Subsystem);

	/** The ip & port that the host is listening on (valid for LAN/GameServer) */
	TSharedPtr<class FInternetAddr> HostAddr;
	/** Unique Id for this session */
	FUniqueNetIdNullRef SessionId;

public:

	virtual ~FOnlineSessionInfoNull() = default;

 	bool operator==(const FOnlineSessionInfoNull& Other) const
 	{
 		return false;
 	}

	virtual const uint8* GetBytes() const override
	{
		return NULL;
	}

	virtual int32 GetSize() const override
	{
		return sizeof(uint64) + sizeof(TSharedPtr<class FInternetAddr>);
	}

	virtual bool IsValid() const override
	{
		// LAN case
		return HostAddr.IsValid() && HostAddr->IsValid();
	}

	virtual FString ToString() const override
	{
		return SessionId->ToString();
	}

	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("HostIP: %s SessionId: %s"), 
			HostAddr.IsValid() ? *HostAddr->ToString(true) : TEXT("INVALID"), 
			*SessionId->ToDebugString());
	}

	virtual const FUniqueNetId& GetSessionId() const override
	{
		return *SessionId;
	}
};
