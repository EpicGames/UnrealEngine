// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineTitleFileNull.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MonitoredProcess.h"
#include "HAL/FileManager.h"
#include "OnlineSubsystemNull.h"


void FTitleFileNullAsyncLoadLocalDevFile::DoWork()
{
	// load file from disk
	bool bLoadedFile = false;

	FArchive* Reader = IFileManager::Get().CreateFileReader(*LocalDevFilePath, FILEREAD_Silent);
	if (Reader)
	{
		int64 SizeToRead = Reader->TotalSize();
		FileData.Reset(SizeToRead);
		FileData.AddUninitialized(SizeToRead);

		uint8* FileDataPtr = FileData.GetData();

		static const int64 ChunkSize = 100 * 1024;

		int64 TotalBytesRead = 0;
		while (SizeToRead > 0)
		{
			int64 Val = FMath::Min(SizeToRead, ChunkSize);
			Reader->Serialize(FileDataPtr + TotalBytesRead, Val);
			BytesRead.Add(Val);
			TotalBytesRead += Val;
			SizeToRead -= Val;
		}

		ensure(SizeToRead == 0 && Reader->TotalSize() == TotalBytesRead);
		bLoadedFile = Reader->Close();
		delete Reader;
	}
	if (bLoadedFile)
	{
		UE_LOG_ONLINE(Verbose, TEXT("async local ini read completed for file '%s'"), *LocalDevFilePath);
	}
	else
	{
		UE_LOG_ONLINE(Verbose, TEXT("async local ini read failed for file '%s'"), *LocalDevFilePath);
	}
}

void FOnlineTitleFileNull::InitializeLocalFilesFromPerforceIfNeeded()
{
	constexpr TCHAR PerforceProtocol[] = TEXT("p4://");
	constexpr int32 ProtocolLength = UE_ARRAY_COUNT(PerforceProtocol) - 1;
	constexpr TCHAR HotfixDirBaseName[] = TEXT("Hotfix ");

	if (LocalPath.ToLower().StartsWith(PerforceProtocol))
	{
#if PLATFORM_WINDOWS && !UE_BUILD_SHIPPING
		TArray<FString> OldHotfixDirs;
		FString FileMatch = FPaths::RemoveDuplicateSlashes(FString::Printf(TEXT("%s/%s*"), *FPaths::EngineIntermediateDir(), HotfixDirBaseName));
		IFileManager::Get().FindFiles(OldHotfixDirs, *FileMatch, /*Files=*/false, /*Dirs=*/true);
		for (const FString& DirectoryName : OldHotfixDirs)
		{
			FString PidString = DirectoryName;
			PidString.RemoveFromStart(HotfixDirBaseName);
			int Pid = 0;
			LexFromString(Pid, *PidString);
			if (Pid != 0 && !FPlatformProcess::IsApplicationRunning(Pid))
			{
				FString FullPath = FPaths::EngineIntermediateDir() / DirectoryName;
				IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*FullPath);
			}
		}

		// If the path is a perforce path, attempt to use perforce to print the files to a local temp directory
		FString TempDirectory = FPaths::ConvertRelativePathToFull(FPaths::RemoveDuplicateSlashes(FString::Printf(TEXT("%s/%s%d"), *FPaths::EngineIntermediateDir(), HotfixDirBaseName, FPlatformProcess::GetCurrentProcessId())));
		FPaths::NormalizeDirectoryName(TempDirectory);
		FPaths::MakePlatformFilename(TempDirectory);
		if (IPlatformFile::GetPlatformPhysical().CreateDirectory(*TempDirectory))
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_P4FetchHotfixFiles);
			bShouldDeleteLocalPath = true;
			UE::HAL::FProcessStartInfo PerforceStartInfo;
			PerforceStartInfo.bHidden = true;
			const TCHAR* PerforcePath = (*LocalPath) + ProtocolLength;
			FString ArgumentsStr = FString::Printf(TEXT("/c \"p4 print -o ^\"%s\\*.ini^\" //%s/*.ini\""), *TempDirectory, PerforcePath);
			TArray<FString> Results;
			bool bSuccess = FMonitoredProcess::RunProcessSynchronous(TEXT("cmd"), ArgumentsStr, Results, /*Timeout=*/30.f);
			FString OutputString = FString::Join(Results, TEXT("\n"));
			if (bSuccess && !OutputString.Contains(TEXT("no such file(s)")))
			{
				LocalPath = TempDirectory;
			}
			else
			{
				bSuccess = false;
				IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*TempDirectory);
				bShouldDeleteLocalPath = false;
				ensureMsgf(false, TEXT("Failed to retrieve hotfix files from %s. Perforce result: %s"), *LocalPath, *OutputString);
				LocalPath.Reset();
			}
		}
#else
		checkf(PLATFORM_WINDOWS, TEXT("Unexpected p4:// TitleFilePath on unsupported platform. Implement FOnlineTitleFileNull::InitializeLocalFilesFromPerforceIfNeeded() if required."));
#endif //PLATFORM_WINDOWS
	}
}

FOnlineTitleFileNull::FOnlineTitleFileNull()
{
	// check for the option that is needed for this to work (if it's not, EnumerateFiles will do nothing
	FParse::Value(FCommandLine::Get(), TEXT("TitleFileLocalPath="), LocalPath);
	bShouldDeleteLocalPath = false;

	InitializeLocalFilesFromPerforceIfNeeded();
}

FOnlineTitleFileNull::~FOnlineTitleFileNull()
{
	if (bShouldDeleteLocalPath)
	{
		IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*LocalPath);
		LocalPath.Reset();
	}
}

bool FOnlineTitleFileNull::GetFileContents(const FString& FileName, TArray<uint8>& FileContents)
{
	const TArray<FCloudFile>* FilesPtr = &Files;
	if (FilesPtr != NULL)
	{
		for (TArray<FCloudFile>::TConstIterator It(*FilesPtr); It; ++It)
		{
			if (It->FileName == FileName)
			{
				FileContents = It->Data;
				return true;
			}
		}
	}
	return false;
}

bool FOnlineTitleFileNull::ClearFiles()
{
	for (int Idx = 0; Idx < Files.Num(); Idx++)
	{
		if (Files[Idx].AsyncState == EOnlineAsyncTaskState::InProgress)
		{
			UE_LOG_ONLINE(Warning, TEXT("can't clear all files - '%s' has a pending async task!"), *Files[Idx].FileName);
			return false;
		}
	}
	// remove all cached file entries
	Files.Empty();
	return true;
}

bool FOnlineTitleFileNull::ClearFile(const FString& FileName)
{
	for (int Idx = 0; Idx < Files.Num(); Idx++)
	{
		if (Files[Idx].FileName == FileName)
		{
			if (Files[Idx].AsyncState == EOnlineAsyncTaskState::InProgress)
			{
				UE_LOG_ONLINE(Warning, TEXT("can't clear file '%s' - it has a pending async task!"), *Files[Idx].FileName);
				return false;
			}
			else
			{
				Files.RemoveAt(Idx);
				return true;
			}
		}
	}
	return false;
}

void FOnlineTitleFileNull::DeleteCachedFiles(bool bSkipEnumerated)
{
	// no cache to delete!
}

bool FOnlineTitleFileNull::EnumerateFiles(const FPagedQuery& Page)
{
	if (LocalPath.IsEmpty())
	{
		TriggerOnEnumerateFilesCompleteDelegates(false, TEXT("-TitleFileLocalPath= was not specified on the commandline."));
		return false;
	}

	if (!IFileManager::Get().DirectoryExists(*LocalPath))
	{
		ensureMsgf(false, TEXT("TitleFilePath does not exist. Path was: %s"), *LocalPath);
		TriggerOnEnumerateFilesCompleteDelegates(false, TEXT("-TitleFileLocalPath= was specified, but does not exist."));
		return false;
	}

	static int32 sFakeHash = 0;
	//		bIsLoadingLocalDevFiles = true;
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *LocalPath, TEXT("*.*"));
	UE_LOG_ONLINE(Verbose, TEXT("EnumerateFiles using local file path founds %d files at '%s'"), FoundFiles.Num(), *LocalPath);

	const FName NAME_SHA1(TEXT("SHA1"));

	FileHeaders.Empty();
	for (const FString& CurrFile : FoundFiles)
	{
		FString FullPath = FPaths::Combine(LocalPath, CurrFile);
		uint8 Hash[20];
		FSHA1::GetFileSHAHash(*FullPath, Hash, false);

		FCloudFileHeader FileHeader;
		FileHeader.Hash = FString::Printf(TEXT("%d"), ++sFakeHash);
		FileHeader.HashType = NAME_SHA1;
		FileHeader.DLName = FullPath;
		FileHeader.FileName = CurrFile;
		FileHeader.URL = "File://" + FullPath;
		FileHeader.ChunkID = 0;
		int64 FileSize = IFileManager::Get().FileSize(*FullPath);
		check(FileSize < INT_MAX);
		FileHeader.FileSize = (int32)(FileSize);
		// FileHeader.ExternalStorageIds; Leave empty

		FileHeaders.Add(FileHeader);
	}
	// Note: In the canonical cloud file case, this would happen at a latter time and not be on the same stack
	// that EnumerateFiles is called on.
	TriggerOnEnumerateFilesCompleteDelegates(true, TEXT(""));
	return true;
}

void FOnlineTitleFileNull::GetFileList(TArray<FCloudFileHeader>& OutFiles)
{
	TArray<FCloudFileHeader>* FilesPtr = &FileHeaders;
	if (FilesPtr != NULL)
	{
		OutFiles = *FilesPtr;
	}
	else
	{
		OutFiles.Empty();
	}
}

bool FOnlineTitleFileNull::ReadFile(const FString& FileName)
{
	bool bStarted = false;

	FCloudFile* CloudFile = GetCloudFile(FileName, true);
	if (CloudFile->AsyncState == EOnlineAsyncTaskState::InProgress)
	{
		UE_LOG_ONLINE(Verbose, TEXT("file read (local) is already in progress for file '%s'"), *FileName);
		return true;
	}

	FCloudFileHeader* CloudFileHeader = GetCloudFileHeader(FileName);
	if (CloudFileHeader != nullptr)
	{
		FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* NewRead =
			new FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>(CloudFileHeader->FileName, CloudFileHeader->DLName);
		AsyncLocalDevReads.Add(NewRead);
		CloudFile->AsyncState = EOnlineAsyncTaskState::InProgress;
		NewRead->StartBackgroundTask();
		bStarted = true;
	}
	return bStarted;
}

void FOnlineTitleFileNull::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineTitleFileMcpV2_Tick);
	LLM_SCOPE_BYTAG(OnlineSubsystem);

	TArray<int32> ItemsToRemove;
	ItemsToRemove.Reserve(AsyncLocalDevReads.Num());

	// Local files, developer flow
	for (int32 TaskIdx = 0; TaskIdx < AsyncLocalDevReads.Num(); TaskIdx++)
	{
		FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* ReadTask = AsyncLocalDevReads[TaskIdx];
		if (ReadTask->IsDone())
		{
			ItemsToRemove.Add(TaskIdx);
			FinishReadFileLocalDevFile(ReadTask);
		}
	}

	// Clean up any tasks that were completed
	for (int32 ItemIdx = ItemsToRemove.Num() - 1; ItemIdx >= 0; ItemIdx--)
	{
		const int32 TaskIdx = ItemsToRemove[ItemIdx];
		FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* TaskToDelete = AsyncLocalDevReads[TaskIdx];
		const FTitleFileNullAsyncLoadLocalDevFile& TaskInfo = TaskToDelete->GetTask();
		UE_LOG_ONLINE(VeryVerbose, TEXT("removed local dev file read task for: %s (bytes read: %lld)"), *TaskInfo.LocalDevFileName, TaskInfo.BytesRead.GetValue());
		delete TaskToDelete;
		AsyncLocalDevReads.RemoveAtSwap(TaskIdx);
	}
	ItemsToRemove.Empty();
}

FCloudFileHeader* FOnlineTitleFileNull::GetCloudFileHeader(const FString& FileName)
{
	FCloudFileHeader* CloudFileHeader = NULL;

	for (int Idx = 0; Idx < FileHeaders.Num(); Idx++)
	{
		if (FileHeaders[Idx].DLName == FileName)
		{
			CloudFileHeader = &FileHeaders[Idx];
			break;
		}
	}

	return CloudFileHeader;
}

FCloudFile* FOnlineTitleFileNull::GetCloudFile(const FString& FileName, bool bCreateIfMissing)
{
	FCloudFile* CloudFile = NULL;
	for (int Idx = 0; Idx < Files.Num(); Idx++)
	{
		if (Files[Idx].FileName == FileName)
		{
			CloudFile = &Files[Idx];
			break;
		}
	}
	if (CloudFile == NULL && bCreateIfMissing)
	{
		CloudFile = new(Files)FCloudFile(FileName);
	}
	return CloudFile;
}

/** Completes the async operation for the local dev file read. Similar to above but without any cloud connection. */
void FOnlineTitleFileNull::FinishReadFileLocalDevFile(FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* Task)
{
	const FTitleFileNullAsyncLoadLocalDevFile& TaskInfo = Task->GetTask();
	FCloudFileHeader* CloudFileHeader = GetCloudFileHeader(TaskInfo.LocalDevFilePath);
	FCloudFile* CloudFile = GetCloudFile(TaskInfo.LocalDevFilePath, true);
	if (CloudFileHeader != nullptr && CloudFile != nullptr)
	{
		CloudFile->Data = TaskInfo.FileData;
		CloudFile->AsyncState = EOnlineAsyncTaskState::Done;
		CloudFileHeader->Hash = TaskInfo.LocalHashStr;
		TriggerOnReadFileCompleteDelegates(true, TaskInfo.LocalDevFilePath);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("failed to read '%s' from given local Dev path!"), *TaskInfo.LocalDevFileName);
	}
}
