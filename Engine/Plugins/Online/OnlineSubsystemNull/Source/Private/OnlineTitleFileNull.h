// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Interfaces/OnlineTitleFileInterface.h"

class FOnlineSubsystemNull;

class FTitleFileNullAsyncLoadLocalDevFile :
	public FNonAbandonableTask
{
public:
	/** File data loaded for the async read */
	TArray<uint8> FileData;
	/** Amount of data read from the file to be owned/referenced by the game thread */
	FThreadSafeCounter64 BytesRead;
	/** The original name of the file being read */
	const FString LocalDevFileName;
	/** The full path file being read */
	const FString LocalDevFilePath;
	/** Hash Calculated after file is loaded */
	FString LocalHashStr;

	FTitleFileNullAsyncLoadLocalDevFile(const FString& FileName, const FString& FullPath) :
		LocalDevFileName(FileName),
		LocalDevFilePath(FullPath)
	{

	}

	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTitleFileNullAsyncLoadLocalDevFile, STATGROUP_ThreadPoolAsyncTasks);
	}

};

class FOnlineTitleFileNull : public IOnlineTitleFile, public TSharedFromThis<FOnlineTitleFileNull, ESPMode::ThreadSafe>
{
public:

	FOnlineTitleFileNull();
	virtual ~FOnlineTitleFileNull();


	virtual bool GetFileContents(const FString& FileName, TArray<uint8>& FileContents) override;
	virtual bool ClearFiles() override;
	virtual bool ClearFile(const FString& FileName) override;
	virtual void DeleteCachedFiles(bool bSkipEnumerated) override;
	virtual bool EnumerateFiles(const FPagedQuery& Page) override;
	virtual void GetFileList(TArray<FCloudFileHeader>& OutFiles) override;
	virtual bool ReadFile(const FString& FileName) override;

	void Tick(float DeltaTime);

protected:
	FString LocalPath;
	TArray<FCloudFileHeader> FileHeaders;
	TArray<FCloudFile> Files;
	bool bShouldDeleteLocalPath;

	/** Holds the outstanding task for hitch free loading of local Dev files. */
	TArray<FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* > AsyncLocalDevReads;


	// these are all copied from OnlineTitleMcp, and maybe could be shared somehow
	FCloudFileHeader* GetCloudFileHeader(const FString& FileName);
	FCloudFile* GetCloudFile(const FString& FileName, bool bCreateIfMissing);
	/** Completes the async operation for the local dev file read. Similar to above but without any cloud connection. */
	void FinishReadFileLocalDevFile(FAsyncTask<FTitleFileNullAsyncLoadLocalDevFile>* Task);

	// Special handling for "p4://" title file paths
	void InitializeLocalFilesFromPerforceIfNeeded();
};

using FOnlineTitleFileNullPtr = TSharedPtr<FOnlineTitleFileNull, ESPMode::ThreadSafe>;
