// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "OnlineSubsystemNull.h"
#include "OnlineSessionInterfaceNull.h"
#include "OnlineSessionSettings.h"
#include "Online/OnlineSessionNames.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Verifies that the Null OSS LAN session search can discover multiple sessions
 * hosted by a single instance. This is a regression test for UE-178220 where
 * FLANSession::Tick's per-host GUID caching prevented all but the first session
 * from being found.
 *
 * The test creates two FOnlineSubsystemNull instances (Host and Searcher),
 * registers two sessions on the Host, performs a LAN search from the Searcher,
 * and asserts that both sessions appear in the results with distinct SessionIds.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FOnlineSessionNullMultiSessionLANDiscovery,
	"System.Engine.Online.OnlineSubsystemNull.Session.MultiSessionLANDiscovery",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FOnlineSessionNullMultiSessionLANDiscovery::RunTest(const FString& Parameters)
{
	// Create two subsystem instances with distinct names so they get separate LAN beacons.
	TSharedRef<FOnlineSubsystemNull> HostSubsystem = MakeShared<FOnlineSubsystemNull>(FName(TEXT("TestHost")));
	TSharedRef<FOnlineSubsystemNull> SearchSubsystem = MakeShared<FOnlineSubsystemNull>(FName(TEXT("TestSearcher")));

	if (!HostSubsystem->Init() || !SearchSubsystem->Init())
	{
		AddError(TEXT("Failed to initialize one or both Null subsystem instances."));
		HostSubsystem->Shutdown();
		SearchSubsystem->Shutdown();
		return false;
	}

	IOnlineSessionPtr HostSessionInt = HostSubsystem->GetSessionInterface();
	IOnlineSessionPtr SearchSessionInt = SearchSubsystem->GetSessionInterface();

	if (!HostSessionInt.IsValid() || !SearchSessionInt.IsValid())
	{
		AddError(TEXT("Failed to get session interface from one or both subsystems."));
		HostSessionInt.Reset();
		SearchSessionInt.Reset();
		HostSubsystem->Shutdown();
		SearchSubsystem->Shutdown();
		return false;
	}

	// --- Host: create two LAN sessions ---

	FOnlineSessionSettings SessionSettingsA;
	SessionSettingsA.NumPublicConnections = 4;
	SessionSettingsA.bIsLANMatch = true;
	SessionSettingsA.bShouldAdvertise = true;
	SessionSettingsA.bAllowJoinInProgress = true;
	SessionSettingsA.bAllowJoinViaPresence = true;

	FOnlineSessionSettings SessionSettingsB;
	SessionSettingsB.NumPublicConnections = 4;
	SessionSettingsB.bIsLANMatch = true;
	SessionSettingsB.bShouldAdvertise = true;
	SessionSettingsB.bAllowJoinInProgress = true;
	SessionSettingsB.bAllowJoinViaPresence = true;

	const FName SessionNameA = NAME_GameSession;
	const FName SessionNameB = FName(TEXT("PartySession"));

	bool bCreatedA = HostSessionInt->CreateSession(0, SessionNameA, SessionSettingsA);
	bool bCreatedB = HostSessionInt->CreateSession(0, SessionNameB, SessionSettingsB);

	if (!bCreatedA || !bCreatedB)
	{
		AddError(FString::Printf(TEXT("Failed to create host sessions (A=%d, B=%d)."), bCreatedA, bCreatedB));
		HostSessionInt.Reset();
		SearchSessionInt.Reset();
		HostSubsystem->Shutdown();
		SearchSubsystem->Shutdown();
		return false;
	}

	// --- Searcher: start a LAN search ---

	TSharedRef<FOnlineSessionSearch> SearchSettings = MakeShared<FOnlineSessionSearch>();
	SearchSettings->bIsLanQuery = true;
	SearchSettings->MaxSearchResults = 10;

	bool bSearchStarted = SearchSessionInt->FindSessions(0, SearchSettings);
	if (!bSearchStarted)
	{
		AddError(TEXT("Failed to start LAN session search."));
		HostSessionInt->DestroySession(SessionNameA);
		HostSessionInt->DestroySession(SessionNameB);
		HostSessionInt.Reset();
		SearchSessionInt.Reset();
		HostSubsystem->Shutdown();
		SearchSubsystem->Shutdown();
		return false;
	}

	// --- Tick both subsystems until the search completes or we time out ---

	const double TimeoutSeconds = 8.0;
	const double StartTime = FPlatformTime::Seconds();
	const float TickDelta = 0.016f; // ~60fps

	while (SearchSettings->SearchState == EOnlineAsyncTaskState::InProgress)
	{
		HostSubsystem->Tick(TickDelta);
		SearchSubsystem->Tick(TickDelta);

		FPlatformProcess::Sleep(TickDelta);

		if (FPlatformTime::Seconds() - StartTime > TimeoutSeconds)
		{
			AddError(TEXT("LAN search timed out before completing."));
			break;
		}
	}

	// --- Validate results ---

	const int32 NumResults = SearchSettings->SearchResults.Num();
	UE_LOGF(LogTemp, Log, "[UE-178220 Test] Search returned %d result(s).", NumResults);

	if (NumResults < 2)
	{
		AddError(FString::Printf(
			TEXT("Expected at least 2 search results but got %d. "
			     "This indicates the multi-session LAN discovery fix may not be working."),
			NumResults));
	}
	else
	{
		// Verify distinct SessionIds
		TSet<FString> UniqueSessionIds;
		for (const FOnlineSessionSearchResult& Result : SearchSettings->SearchResults)
		{
			FString SessionId = Result.Session.GetSessionIdStr();
			UE_LOGF(LogTemp, Log, "[UE-178220 Test] Found session: %ls", *SessionId);
			UniqueSessionIds.Add(SessionId);
		}

		if (UniqueSessionIds.Num() < 2)
		{
			AddError(TEXT("Search results contain duplicate SessionIds - expected distinct sessions."));
		}
		else
		{
			UE_LOGF(LogTemp, Log, "[UE-178220 Test] PASSED: Found %d unique sessions.", UniqueSessionIds.Num());
		}
	}

	// --- Cleanup ---
	// Release interface refs before Shutdown(), which asserts unique ownership.

	HostSessionInt->DestroySession(SessionNameA);
	HostSessionInt->DestroySession(SessionNameB);
	HostSessionInt.Reset();
	SearchSessionInt.Reset();

	HostSubsystem->Shutdown();
	SearchSubsystem->Shutdown();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
