// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VoiceInterfaceImpl.h"

class FOnlineVoiceImplNull : public FOnlineVoiceImpl {
PACKAGE_SCOPE:
	FOnlineVoiceImplNull() : FOnlineVoiceImpl()
	{};

public:
	/** Constructor */
	FOnlineVoiceImplNull(class IOnlineSubsystem* InOnlineSubsystem) :
		FOnlineVoiceImpl(InOnlineSubsystem)
	{
	};

	virtual ~FOnlineVoiceImplNull() override = default;
};

typedef TSharedPtr<FOnlineVoiceImpl, ESPMode::ThreadSafe> FOnlineVoiceImplPtr;
