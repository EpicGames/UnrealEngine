// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemNullPackage.h"

#define UE_API ONLINESUBSYSTEMNULL_API

class FThreadSafeCounter;

class FOnlineAchievementsNull;
class FOnlineIdentityNull;
class FOnlineLeaderboardsNull;
class FOnlineSessionNull;
class FOnlineVoiceImpl;

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineSessionNull, ESPMode::ThreadSafe> FOnlineSessionNullPtr;
typedef TSharedPtr<class FOnlineProfileNull, ESPMode::ThreadSafe> FOnlineProfileNullPtr;
typedef TSharedPtr<class FOnlineFriendsNull, ESPMode::ThreadSafe> FOnlineFriendsNullPtr;
typedef TSharedPtr<class FOnlineUserCloudNull, ESPMode::ThreadSafe> FOnlineUserCloudNullPtr;
typedef TSharedPtr<class FOnlineLeaderboardsNull, ESPMode::ThreadSafe> FOnlineLeaderboardsNullPtr;
typedef TSharedPtr<class FOnlineExternalUINull, ESPMode::ThreadSafe> FOnlineExternalUINullPtr;
typedef TSharedPtr<class FOnlineIdentityNull, ESPMode::ThreadSafe> FOnlineIdentityNullPtr;
typedef TSharedPtr<class FOnlineAchievementsNull, ESPMode::ThreadSafe> FOnlineAchievementsNullPtr;
typedef TSharedPtr<class FOnlineStoreV2Null, ESPMode::ThreadSafe> FOnlineStoreV2NullPtr;
typedef TSharedPtr<class FOnlinePurchaseNull, ESPMode::ThreadSafe> FOnlinePurchaseNullPtr;
typedef TSharedPtr<class FMessageSanitizerNull, ESPMode::ThreadSafe> FMessageSanitizerNullPtr;
typedef TSharedPtr<class FOnlineTitleFileNull, ESPMode::ThreadSafe> FOnlineTitleFileNullPtr;
#if WITH_ENGINE
typedef TSharedPtr<class FOnlineVoiceImpl, ESPMode::ThreadSafe> FOnlineVoiceImplPtr;
#endif //WITH_ENGINE

/**
 *	OnlineSubsystemNull - Implementation of the online subsystem for Null services
 */
class FOnlineSubsystemNull : 
	public FOnlineSubsystemImpl
{

public:

	virtual ~FOnlineSubsystemNull() = default;

	// IOnlineSubsystem

	UE_API virtual IOnlineSessionPtr GetSessionInterface() const override;
	UE_API virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	UE_API virtual IOnlinePartyPtr GetPartyInterface() const override;
	UE_API virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	UE_API virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	UE_API virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	UE_API virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	UE_API virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	UE_API virtual IOnlineVoicePtr GetVoiceInterface() const override;
	UE_API virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;	
	UE_API virtual IOnlineTimePtr GetTimeInterface() const override;
	UE_API virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	UE_API virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	UE_API virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	UE_API virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	UE_API virtual IOnlineEventsPtr GetEventsInterface() const override;
	UE_API virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	UE_API virtual IOnlineSharingPtr GetSharingInterface() const override;
	UE_API virtual IOnlineUserPtr GetUserInterface() const override;
	UE_API virtual IOnlineMessagePtr GetMessageInterface() const override;
	UE_API virtual IOnlinePresencePtr GetPresenceInterface() const override;
	UE_API virtual IOnlineChatPtr GetChatInterface() const override;
	UE_API virtual IOnlineStatsPtr GetStatsInterface() const override;
	UE_API virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	UE_API virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	UE_API virtual IMessageSanitizerPtr GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const override;

	UE_API virtual bool Init() override;
	UE_API virtual bool Shutdown() override;
	UE_API virtual FString GetAppId() const override;
	UE_API virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	UE_API virtual FText GetOnlineServiceName() const override;

	// FTSTickerObjectBase
	
	UE_API virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemNull

PACKAGE_SCOPE:

	/** Only the factory makes instances */
	FOnlineSubsystemNull() = delete;
	explicit FOnlineSubsystemNull(FName InInstanceName) :
		FOnlineSubsystemImpl(NULL_SUBSYSTEM, InInstanceName),
		SessionInterface(nullptr),
		VoiceInterface(nullptr),
		bVoiceInterfaceInitialized(false),
		LeaderboardsInterface(nullptr),
		IdentityInterface(nullptr),
		AchievementsInterface(nullptr),
		StoreV2Interface(nullptr),
		MessageSanitizerInterface(nullptr),
		TitleFileInterface(nullptr),
		OnlineAsyncTaskThreadRunnable(nullptr),
		OnlineAsyncTaskThread(nullptr)
	{}

	// Options for emulating different types of online platforms, these are settable via OSSNull cvars or in the [OnlineSubsystemNull] config section

	/** True if it should login the first user at startup like single-user platforms, false to only login when requested */
	static UE_API bool bAutoLoginAtStartup;

	/** True if it should support an external UI interface */
	static UE_API bool bSupportExternalUI;

	/** True if login requires calling ShowLoginUI on the externalUI, depends on SupportExternalUI */
	static UE_API bool bRequireShowLoginUI;

	/** True if the user index should change during login UI to emulate a platform user change */
	static UE_API bool bForceShowLoginUIUserChange;

	/** True if login should require a user/pass to act like an external service, false to match most platforms and use the default */
	static UE_API bool bRequireLoginCredentials;

	/** True if login name should include the local user number, which allows different stable IDs per user num */
	static UE_API bool bAddUserNumToNullId;

	/** True if it should use a system-stable null Id for login, same as -StableNullID on command line */
	static UE_API bool bForceStableNullId;

	/** True if it should fail faked network queries and act like an offline system */
	static UE_API bool bForceOfflineMode;

	/** True if the first login only counts as local login, a second is required for online access */
	static UE_API bool bOnlineRequiresSecondLogin;

private:

	/** Interface to the session services */
	FOnlineSessionNullPtr SessionInterface;

	/** Interface for voice communication */
	mutable IOnlineVoicePtr VoiceInterface;

	/** Interface for voice communication */
	mutable bool bVoiceInterfaceInitialized;

	/** Interface to the leaderboard services */
	FOnlineLeaderboardsNullPtr LeaderboardsInterface;

	/** Interface to the identity registration/auth services */
	FOnlineIdentityNullPtr IdentityInterface;

	/** Interface to the identity registration/auth services */
	FOnlineExternalUINullPtr ExternalUIInterface;

	/** Interface for achievements */
	FOnlineAchievementsNullPtr AchievementsInterface;

	/** Interface for store */
	FOnlineStoreV2NullPtr StoreV2Interface;

	/** Interface for purchases */
	FOnlinePurchaseNullPtr PurchaseInterface;

	/** Interface for message sanitizing */
	FMessageSanitizerNullPtr MessageSanitizerInterface;

	/** Interface for title file storage */
	FOnlineTitleFileNullPtr TitleFileInterface;

	/** Online async task runnable */
	class FOnlineAsyncTaskManagerNull* OnlineAsyncTaskThreadRunnable;

	/** Online async task thread */
	class FRunnableThread* OnlineAsyncTaskThread;

	// task counter, used to generate unique thread names for each task
	static UE_API FThreadSafeCounter TaskCounter;
};

typedef TSharedPtr<FOnlineSubsystemNull, ESPMode::ThreadSafe> FOnlineSubsystemNullPtr;

#undef UE_API
