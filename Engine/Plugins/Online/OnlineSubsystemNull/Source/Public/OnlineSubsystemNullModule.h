// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformCrt.h"
#include "Modules/ModuleInterface.h"

/**
 * Online subsystem module class  (Null Implementation)
 * Code related to the loading of the Null module
 */
class FOnlineSubsystemNullModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryNull* NullFactory;

public:

	FOnlineSubsystemNullModule() : 
		NullFactory(NULL)
	{}

	virtual ~FOnlineSubsystemNullModule() = default;

	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};
